// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class PhysicsUtilities : ModuleRules
{
	public PhysicsUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"PhysicsCore",
				"Chaos"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"MeshUtilitiesCommon",
				"MeshUtilitiesEngine",
				"GeometryCore"		// For mesh to level set conversion
            }
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "VHACD");
	}
}
