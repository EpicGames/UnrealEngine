// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ConvexDecompTool.cpp: Utility for turning graphics mesh into convex hulls.
=============================================================================*/

#include "ConvexDecompTool.h"

#include "Misc/FeedbackContext.h"
#include "PhysicsEngine/ConvexElem.h"

#include "ThirdParty/VHACD/public/VHACD.h"
#include "ThirdParty/VHACD/inc/btAlignedAllocator.h"

#include "HAL/UnrealMemory.h"
#include "PhysicsEngine/BodySetup.h"
#include "Async/Async.h"

DEFINE_LOG_CATEGORY_STATIC(LogConvexDecompTool, Log, All);

using namespace VHACD;

class FVHACDProgressCallback : public IVHACD::IUserCallback
{
public:
	FVHACDProgressCallback(void) {}
	~FVHACDProgressCallback() {};

	void Update(const double overallProgress, const double stageProgress, const double operationProgress, const char * const stage,	const char * const    operation)
	{
		if (IsInGameThread())
		{
			FString StatusString = FString::Printf(TEXT("Processing [%s]..."), ANSI_TO_TCHAR(stage));
			GWarn->StatusUpdate(stageProgress*10.f, 1000, FText::FromString(StatusString));
		}
	};
};

//#define DEBUG_VHACD
#ifdef DEBUG_VHACD
class FVHACDLogger : public IVHACD::IUserLogger
{
public:
	virtual ~FVHACDLogger() {};
	virtual void Log(const char * const msg) override
	{
		UE_LOGF(LogConvexDecompTool, Log, "VHACD: %ls", ANSI_TO_TCHAR(msg));
	}
};
static FVHACDLogger VHACDLogger;
#endif // DEBUG_VHACD

#if CPUPROFILERTRACE_ENABLED
class FVHACDProfiler : public IVHACD::IUserProfiler
{
public:
	virtual ~FVHACDProfiler() {};

	virtual unsigned int AllocTagId(const char* const tagName) override
	{
		return FCpuProfilerTrace::OutputEventType(tagName);
	}

	virtual void EnterTag(unsigned int tagId) override
	{
		FCpuProfilerTrace::OutputBeginEvent(tagId);
	}

	virtual void ExitTag() override
	{
		FCpuProfilerTrace::OutputEndEvent();
	}

	virtual void Bookmark(const char* text) override
	{
		TRACE_BOOKMARK(TEXT("%s"), ANSI_TO_TCHAR(text));
	}
};
static FVHACDProfiler VHACDProfiler;
#endif

class FVHACDTaskRunner : public IVHACD::IUserTaskRunner
{
public:
	virtual ~FVHACDTaskRunner() {};

	virtual void* StartTask(std::function<void()> func)
	{
		// Capturing a std::function and passing it to Async.
		// This works today but if Async() or TFuture start to assume they can trivially relocate
		// callbacks (e.g. by storing them in a TArray) then this capture will need revising.
		// Unlike TFunction, std::function is not guaranteed to be trivially relocatable.
		return new TFuture<void>(Async(EAsyncExecution::ThreadPool, [func](){func();}));
	}

	virtual void JoinTask(void* Task)
	{
		TFuture<void>* Future = (TFuture<void>*)Task;
		Future->Wait();
		delete Future;
	}
};

static FVHACDTaskRunner VHACDTaskRunner;

static void* btAlignedAllocImpl(size_t size, int32_t alignment)
{
	return FMemory::Malloc(size, alignment);
}

static void btAlignedFreeImpl(void* memblock)
{
	FMemory::Free(memblock);
}

static void* btAllocImpl(size_t size)
{
	return FMemory::Malloc(size);
}

static void btFreeImpl(void* memblock)
{
	FMemory::Free(memblock);
}

static void InitParameters(IVHACD::Parameters &VHACD_Params, uint32 InHullCount, uint32 InMaxHullVerts,uint32 InResolution)
{
	// Override VHACD allocator with ours
	btAlignedAllocSetCustom(btAllocImpl, btFreeImpl);
	btAlignedAllocSetCustomAligned(btAlignedAllocImpl, btAlignedFreeImpl);

#ifdef DEBUG_VHACD
	VHACD_Params.m_logger = &VHACDLogger;
#endif //DEBUG_VHACD
#if CPUPROFILERTRACE_ENABLED
	VHACD_Params.m_profiler = &VHACDProfiler;
#endif
	VHACD_Params.m_taskRunner = &VHACDTaskRunner; // any async tasks will be run on existing UE thread pools.
	VHACD_Params.m_resolution = FMath::Max(InResolution, (uint32)10000); // Maximum number of voxels generated during the voxelization stage (default=100,000, range=10,000-16,000,000)
	VHACD_Params.m_maxNumVerticesPerCH = FMath::Max(InMaxHullVerts, (uint32)4); // Controls the maximum number of triangles per convex-hull (default=64, range=4-1024)
	VHACD_Params.m_concavity = 0;		// Concavity is set to zero so that we consider any concave shape as a potential hull. The number of output hulls is better controlled by recursion depth and the max convex hulls parameter
	VHACD_Params.m_maxConvexHulls = InHullCount;	// The number of convex hulls requested by the artists/designer
	VHACD_Params.m_oclAcceleration = false;
	VHACD_Params.m_minVolumePerCH = 0.003f; // this should be around 1 / (3 * m_resolution ^ (1/3))
	VHACD_Params.m_projectHullVertices = true; // Project the approximate hull vertices onto the original source mesh and use highest precision results possible.
	VHACD_Params.m_pca = 1; // align mesh for more optimal processing of long diagonal meshes
}

void DecomposeMeshToHulls(UBodySetup* InBodySetup, const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, uint32 InHullCount, int32 InMaxHullVerts,uint32 InResolution)
{
	check(InBodySetup != NULL);

	bool bSuccess = false;

	// Validate input by checking bounding box
	FBox VertBox(ForceInit);
	for (const FVector3f& Vert : InVertices)
	{
		VertBox += (FVector)Vert;
	}

	// If box is invalid, or the largest dimension is less than 1 unit, or smallest is less than 0.1, skip trying to generate collision (V-HACD often crashes...)
	if (VertBox.IsValid == 0 || VertBox.GetSize().GetMax() < 1.f || VertBox.GetSize().GetMin() < 0.1f)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(DecomposeMeshToHulls)

	btAlignedAllocSetCustom(btAllocImpl, btFreeImpl);
	btAlignedAllocSetCustomAligned(btAlignedAllocImpl, btAlignedFreeImpl);

	FVHACDProgressCallback VHACD_Callback;
	IVHACD::Parameters VHACD_Params;

	InitParameters(VHACD_Params, InHullCount, InMaxHullVerts,InResolution);

	VHACD_Params.m_callback = &VHACD_Callback;		// callback interface for message/status updates

	if (IsInGameThread())
	{
		GWarn->BeginSlowTask(NSLOCTEXT("ConvexDecompTool", "BeginCreatingCollisionTask", "Creating Collision"), true, false);
	}

	IVHACD* InterfaceVHACD = CreateVHACD();

	const float* const Verts = (float*)InVertices.GetData();
	const unsigned int NumVerts = InVertices.Num();
	const uint32_t* const Tris = (uint32_t*)InIndices.GetData();
	const unsigned int NumTris = InIndices.Num() / 3;

	bSuccess = InterfaceVHACD->Compute(Verts, NumVerts, Tris, NumTris, VHACD_Params);

	if (IsInGameThread())
	{
		GWarn->EndSlowTask();
	}

	if(bSuccess)
	{
		// Clean out old hulls
		InBodySetup->RemoveSimpleCollision();

		// Iterate over each result hull
		int32 NumHulls = InterfaceVHACD->GetNConvexHulls();
		for(int32 HullIdx=0; HullIdx<NumHulls; HullIdx++)
		{
			// Create a new hull in the aggregate geometry
			FKConvexElem ConvexElem;

			IVHACD::ConvexHull Hull;
			InterfaceVHACD->GetConvexHull(HullIdx, Hull);
			for (uint32 VertIdx = 0; VertIdx < Hull.m_nPoints; VertIdx++)
			{
				FVector V;
				V.X = (float)(Hull.m_points[(VertIdx * 3) + 0]);
				V.Y = (float)(Hull.m_points[(VertIdx * 3) + 1]);
				V.Z = (float)(Hull.m_points[(VertIdx * 3) + 2]);

				ConvexElem.VertexData.Add(V);
			}
			ConvexElem.UpdateElemBox();

			InBodySetup->AggGeom.ConvexElems.Add(ConvexElem);
		}

		InBodySetup->InvalidatePhysicsData(); // update GUID
	}

	InterfaceVHACD->Clean();
	InterfaceVHACD->Release();
}

class FDecomposeMeshToHullsAsyncImpl : public IDecomposeMeshToHullsAsync, public IVHACD::IUserCallback
{
public:
	// VHACDJob : Is a small class used to represent one V-HACD operation request.  These are put in a queue and processed in sequential order
	class VHACDJob
	{
	public:
		VHACDJob(UBodySetup* _InBodySetup,
				 TArray<FVector3f>&& InVertices,
				 TArray<uint32>&& InIndices,
				 uint32 InHullCount,
				 int32 InMaxHullVerts,
				 uint32 InResolution = DEFAULT_HACD_VOXEL_RESOLUTION)
			: Vertices(MoveTemp(InVertices)),			// Take ownership of the buffer of vertex positions
			  Triangles(MoveTemp(InIndices)),			// Take ownership of the buffer of triangle indices
			  InBodySetup(_InBodySetup),				// Cache the UBodySetup the convex hull results are to be stored in.
			  HullCount(InHullCount),					// Save the number of convex hulls requested count
			  MaxHullVerts(uint32(InMaxHullVerts)),		// Save the maximum number of vertices per convex hull allowed
			  Resolution(InResolution)					// Save the voxel resolution
		{
		}

		// Start the V-HACD operation
		bool StartJob(IVHACD *InVHACD,IVHACD::IUserCallback *InCallback)
		{
			IVHACD::Parameters VHACD_Params;
			InitParameters(VHACD_Params, HullCount, MaxHullVerts, Resolution);
			VHACD_Params.m_callback = InCallback;
			bool ret = InVHACD->Compute(&Vertices[0].X, Vertices.Num(), Triangles.GetData(), Triangles.Num() / 3, VHACD_Params);
			return ret;
		}

		// Get the results and copy them into the UBodySetup specified by the caller
		void GetResults(IVHACD *InVHACD)
		{
			int32 NumHulls = InVHACD->GetNConvexHulls();
			if (NumHulls)
			{
				// Clean out old hulls
				InBodySetup->RemoveSimpleCollision();
				// Iterate over each result hull
				for (int32 HullIdx = 0; HullIdx < NumHulls; HullIdx++)
				{
					// Create a new hull in the aggregate geometry
					FKConvexElem ConvexElem;
					IVHACD::ConvexHull Hull;
					InVHACD->GetConvexHull(HullIdx, Hull);
					for (uint32 VertIdx = 0; VertIdx < Hull.m_nPoints; VertIdx++)
					{
						FVector V;
						V.X = (float)(Hull.m_points[(VertIdx * 3) + 0]);
						V.Y = (float)(Hull.m_points[(VertIdx * 3) + 1]);
						V.Z = (float)(Hull.m_points[(VertIdx * 3) + 2]);

						ConvexElem.VertexData.Add(V);
					}

					const uint32 NumIndexEntries = Hull.m_nTriangles * 3;
					ConvexElem.IndexData.Reset(Hull.m_nTriangles * 3);
					for(uint32 Index = 0; Index < NumIndexEntries; ++Index)
					{
						ConvexElem.IndexData.Add(Hull.m_triangles[Index]);
					}

					ConvexElem.UpdateElemBox();
					InBodySetup->AggGeom.ConvexElems.Add(ConvexElem);
				}
				InBodySetup->InvalidatePhysicsData(); // update GUID
			}
		}


		VHACDJob		*mNext{ nullptr };						// Next job to perform
private:
		TArray<FVector3f>	Vertices;
		TArray<uint32>		Triangles;
		UBodySetup*			InBodySetup;		// The body we are going to store the results in
		uint32				HullCount;			// The maximum number of convex hulls requested
		uint32				MaxHullVerts;		// The maximum number of vertices per convex hull requested
		uint32				Resolution;			// The voxel resolution to use for the V-HACD operation
	};

	FDecomposeMeshToHullsAsyncImpl(void)
	{
		InterfaceVHACD = VHACD::CreateVHACD_ASYNC(); // create the asynchronous V-HACD instance
	}

	virtual ~FDecomposeMeshToHullsAsyncImpl(void)
	{
		ReleaseVHACD();	// Release any pending jobs
		// Shut down the V-HACD interface
		if (InterfaceVHACD)
		{
			InterfaceVHACD->Clean();
			InterfaceVHACD->Release();
		}
	}


	/**
	*	Utility for turning arbitrary mesh into convex hulls.
	*	@output		InBodySetup			BodySetup that will have its existing hulls removed and replaced with results of decomposition.
	*	@param		InVertices			Array of vertex positions of input mesh
	*	@param		InIndices			Array of triangle indices for input mesh
	*	@param		InAccuracy			Value between 0 and 1, controls how accurate hull generation is
	*	@param		InMaxHullVerts		Number of verts allowed in a hull
	*/
	virtual bool DecomposeMeshToHullsAsyncBegin(UBodySetup* _InBodySetup, TArray<FVector3f>&& InVertices, TArray<uint32>&& InIndices, uint32 InHullCount, int32 InMaxHullVerts, uint32 InResolution = DEFAULT_HACD_VOXEL_RESOLUTION) final
	{
		check(_InBodySetup != NULL);

		bool bSuccess = false;

		// Validate input by checking bounding box
		FBox VertBox(ForceInit);
		for (const FVector3f& Vert : InVertices)
		{
			VertBox += (FVector)Vert;
		}

		// If box is invalid, or the largest dimension is less than 1 unit, or smallest is less than 0.1, skip trying to generate collision (V-HACD often crashes...)
		if (VertBox.IsValid == 0 || VertBox.GetSize().GetMax() < 1.f || VertBox.GetSize().GetMin() < 0.1f)
		{
			return false;
		}

		// Create a VHACD Job instance and add it to the end of the linked list of pending jobs
		VHACDJob *job = new VHACDJob(_InBodySetup, MoveTemp(InVertices), MoveTemp(InIndices), InHullCount, InMaxHullVerts, InResolution);
		// If we already have an active job, add this job to the end of the list
		if (CurrentJob)
		{
			// Add  this new job to the end of the linked list
			VHACDJob *scan = CurrentJob;
			while (scan->mNext )
			{
				scan = scan->mNext;
			}
			scan->mNext = job;
			bSuccess = true;
		}
		else
		{
			bSuccess = job->StartJob(InterfaceVHACD, this); // start the first job!
			CurrentJob = job;
		}

		return bSuccess;
	}


	void ReleaseVHACD(void)
	{
		CurrentStatus = "";
		// Release all pending async jobs
		VHACDJob *job = CurrentJob;
		while (job)
		{
			VHACDJob *next = job->mNext;
			delete job;
			job = next;
		}
		CurrentJob = nullptr;
	}

	virtual bool IsComplete(void) final
	{
		bool ret = true;
		if (CurrentJob)
		{
			if (InterfaceVHACD->IsReady())	// If the last job is finished
			{
				CurrentJob->GetResults(InterfaceVHACD);	// Get the results of that previous V-HACD operation
				VHACDJob *next = CurrentJob->mNext;	// Get the next job pointer
				delete CurrentJob;					// Delete the last job now that it is completed
				CurrentJob = next;					// Assign the new current job pointer and begin work if there is more work to do.
				if (CurrentJob)
				{
					CurrentJob->StartJob(InterfaceVHACD, this); // kick off the next job in sequential order
				}
			}
			ret = CurrentJob ? false : true; // If there is still a CurrentJob then we are not complete; set return code to false
		}
		return ret;
	}

	virtual void Release(void) final
	{
		delete this;
	}

	void Update(const double overallProgress, const double stageProgress, const double operationProgress, const char * const stage, const char * const    operation)
	{
		CurrentStatus = FString::Printf(TEXT("Progress[%0.2f] [%s:%0.2f] [%s:%0.2f]"),
			overallProgress,
			ANSI_TO_TCHAR(stage),
			stageProgress,
			ANSI_TO_TCHAR(operation), operationProgress);
	}

	virtual const FString &GetCurrentStatus(void) const final
	{
		return CurrentStatus;
	}

private:
	VHACDJob		*CurrentJob{ nullptr };
	FString			CurrentStatus;
	IVHACD			*InterfaceVHACD{ nullptr };	// pointer to current asynchronous V-HACD interface
};

/**
*	Utility for turning arbitrary mesh into convex hulls.
*	@output		InBodySetup			BodySetup that will have its existing hulls removed and replaced with results of decomposition.
*	@param		InVertices			Array of vertex positions of input mesh
*	@param		InIndices			Array of triangle indices for input mesh
*	@param		InAccuracy			Value between 0 and 1, controls how accurate hull generation is
*	@param		InMaxHullVerts		Number of verts allowed in a hull
*/
IDecomposeMeshToHullsAsync *CreateIDecomposeMeshToHullAsync(void)
{
	FDecomposeMeshToHullsAsyncImpl *d = new FDecomposeMeshToHullsAsyncImpl;
	return static_cast<IDecomposeMeshToHullsAsync *>(d);
}