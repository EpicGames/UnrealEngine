// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSetHelpers.h"

#include "Chaos/Levelset.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/SkeletalMesh.h"
#include "Implicit/SweepingMeshSDF.h"
#include "Implicit/Solidify.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/LevelSetElem.h"


// Copied from CollisionGeometryConversion.h (Plugins/Runtime/MeshModelingToolset)
static void CreateLevelSetElement(const FVector3d& GridOrigin, const UE::Geometry::FDenseGrid3f& Grid, float CellSize, FKLevelSetElem& LevelSetOut)
{
	const UE::Geometry::FVector3i& GridDims = Grid.GetDimensions();
	TArray<double> OutGridValues;
	OutGridValues.Init(0.0, GridDims[0] * GridDims[1] * GridDims[2]);

	for (int I = 0; I < GridDims[0]; ++I)
	{
		for (int J = 0; J < GridDims[1]; ++J)
		{
			for (int K = 0; K < GridDims[2]; ++K)
			{
				// account for different ordering in Geometry::TDenseGrid3 vs Chaos::TUniformGrid
				const int InBufferIndex = I + GridDims[0] * (J + GridDims[1] * K);
				const int OutBufferIndex = K + GridDims[2] * (J + GridDims[1] * I);
				OutGridValues[OutBufferIndex] = Grid[InBufferIndex];
			}
		}
	}

	const FTransform3d GridTransform = FTransform(GridOrigin);
	FTransform ChaosTransform = GridTransform;
	ChaosTransform.AddToTranslation(-0.5 * CellSize * FVector::One());		// account for grid origin being at cell center vs cell corner
	LevelSetOut.BuildLevelSet(ChaosTransform, OutGridValues, FIntVector(GridDims[0], GridDims[1], GridDims[2]), CellSize);
}

static void CreateLevelSetElement(const FVector3d& GridOrigin, const UE::Geometry::FDenseGrid3f& Grid, float CellSize, Chaos::FLevelSetPtr& LevelSetOut)
{
	const UE::Geometry::FVector3i& GridDims = Grid.GetDimensions();
	TArray<double> OutGridValues;
	OutGridValues.Init(0.0, GridDims[0] * GridDims[1] * GridDims[2]);

	for (int I = 0; I < GridDims[0]; ++I)
	{
		for (int J = 0; J < GridDims[1]; ++J)
		{
			for (int K = 0; K < GridDims[2]; ++K)
			{
				// account for different ordering in Geometry::TDenseGrid3 vs Chaos::TUniformGrid
				const int InBufferIndex = I + GridDims[0] * (J + GridDims[1] * K);
				const int OutBufferIndex = K + GridDims[2] * (J + GridDims[1] * I);
				OutGridValues[OutBufferIndex] = Grid[InBufferIndex];
			}
		}
	}

	const FVector3d GridMin = GridOrigin - 0.5 * CellSize * FVector::One();		// account for grid origin being at cell center vs cell corner
	const FVector3d GridMax = GridMin + CellSize * FVector3d((double)GridDims[0], (double)GridDims[1], (double)GridDims[2]);

	const Chaos::TVec3<int32> ChaosLSGridDims(GridDims[0], GridDims[1], GridDims[2]);
	Chaos::TUniformGrid<Chaos::FReal, 3> LevelSetGrid(GridMin, GridMax, ChaosLSGridDims);
	Chaos::TArrayND<Chaos::FReal, 3> Phi(ChaosLSGridDims, OutGridValues);

	LevelSetOut = Chaos::FLevelSetPtr( new Chaos::FLevelSet(MoveTemp(LevelSetGrid), MoveTemp(Phi), 0));
}

// Copied from FMeshSimpleShapeApproximation::Generate_LevelSets (Plugins/Runtime/GeometryProcessing)
template<typename FLevelSetElemType>
bool CreateLevelSetForMeshInternal(const UE::Geometry::FDynamicMesh3& InMesh, int32 InLevelSetGridResolution, FLevelSetElemType& OutElement)
{
	using UE::Geometry::FDynamicMesh3;
	using UE::Geometry::TSweepingMeshSDF;

	constexpr int32 NumNarrowBandCells = 2;
	constexpr int32 NumExpandCells = 1;

	// Inside SDF.Compute(), extra grid cell are added to account for the narrow band ("NarrowBandMaxDistance") as well as an 
	// outer buffer ("ExpandBounds"). So here we adjust the cell size to make the final output resolution closer to the user-specified resolution.
	const int32 LevelSetGridResolution = FMath::Max(1, InLevelSetGridResolution - 2 * (NumNarrowBandCells + NumExpandCells));

	const UE::Geometry::FAxisAlignedBox3d Bounds = InMesh.GetBounds();
	const double CellSize = Bounds.MaxDim() / LevelSetGridResolution;
	const double ExpandBounds = NumExpandCells * CellSize;

	UE::Geometry::TMeshAABBTree3<FDynamicMesh3> Spatial(&InMesh);

	// Input mesh is likely open, so solidify it first
	UE::Geometry::TFastWindingTree<FDynamicMesh3> FastWinding(&Spatial);
	UE::Geometry::TImplicitSolidify<FDynamicMesh3> Solidify(&InMesh, &Spatial, &FastWinding);
	Solidify.ExtendBounds = ExpandBounds;
	Solidify.MeshCellSize = CellSize;
	Solidify.WindingThreshold = 0.5;
	Solidify.SurfaceSearchSteps = 3;
	Solidify.bSolidAtBoundaries = true;

	const UE::Geometry::FDynamicMesh3 SolidMesh = FDynamicMesh3(&Solidify.Generate());
	Spatial.SetMesh(&SolidMesh, true);

	TSweepingMeshSDF<FDynamicMesh3> SDF;
	SDF.Mesh = &SolidMesh;
	SDF.Spatial = &Spatial;
	SDF.ComputeMode = TSweepingMeshSDF<FDynamicMesh3>::EComputeModes::FullGrid;
	SDF.CellSize = (float)CellSize;
	SDF.NarrowBandMaxDistance = NumNarrowBandCells * CellSize;
	SDF.ExactBandWidth = FMath::CeilToInt32(SDF.NarrowBandMaxDistance / CellSize);
	SDF.ExpandBounds = FVector3d(ExpandBounds);

	if (SDF.Compute(Bounds))
	{
		CreateLevelSetElement((FVector3d)SDF.GridOrigin, SDF.Grid, SDF.CellSize, OutElement);
		return true;
	}

	return false;
}

namespace LevelSetHelpers
{
bool CreateLevelSetForBone(UBodySetup* BodySetup, const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, uint32 InResolution)
{
	check(BodySetup != NULL);

	// Validate input by checking bounding box
	FBox VertBox(ForceInit);
	for (const FVector3f& Vert : InVertices)
	{
		VertBox += (FVector)Vert;
	}

	// If box is invalid, or the largest dimension is less than 1 unit, or smallest is less than 0.1, skip trying to generate collision
	if (VertBox.IsValid == 0 || VertBox.GetSize().GetMax() < 1.f || VertBox.GetSize().GetMin() < 0.1f || InIndices.Num() == 0)
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshToLevelSet)

	// Clean out old hulls
	BodySetup->RemoveSimpleCollision();

	UE::Geometry::FDynamicMesh3 DynamicMesh;
	CreateDynamicMesh(InVertices, InIndices, DynamicMesh);

	FKLevelSetElem LevelSetElement;
	const bool bOK = CreateLevelSetForMesh(DynamicMesh, InResolution, LevelSetElement);

	if (!bOK)
	{
		return false;
	}

	BodySetup->AggGeom.LevelSetElems.Add(LevelSetElement);
	BodySetup->InvalidatePhysicsData(); // update GUID
	return true;
}

void CreateDynamicMesh(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, UE::Geometry::FDynamicMesh3& OutMesh)
{
	OutMesh.Clear();
	for (const FVector3f& Vertex : InVertices)
	{
		OutMesh.AppendVertex(FVector3d(Vertex));
	}

	check(InIndices.Num() % 3 == 0);
	const int32 NumTriangles = InIndices.Num() / 3;
	for (int32 TriangleID = 0; TriangleID < NumTriangles; ++TriangleID)
	{
		const UE::Geometry::FIndex3i Triangle(InIndices[3 * TriangleID], InIndices[3 * TriangleID + 1], InIndices[3 * TriangleID + 2]);
		OutMesh.AppendTriangle(Triangle);
	}
}

bool CreateLevelSetForMesh(const UE::Geometry::FDynamicMesh3& InMesh, int32 InLevelSetGridResolution, FKLevelSetElem& OutElement)
{
	return CreateLevelSetForMeshInternal(InMesh, InLevelSetGridResolution, OutElement);
}

bool CreateLevelSetForMesh(const UE::Geometry::FDynamicMesh3& InMesh, int32 InLevelSetGridResolution, Chaos::FLevelSetPtr& OutElement)
{
	return CreateLevelSetForMeshInternal(InMesh, InLevelSetGridResolution, OutElement);
}

} // namespace LevelSetHelpers
