// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Templates/RefCounting.h"
#include "Chaos/ImplicitFwd.h"

struct FKLevelSetElem;
class UBodySetup;
namespace UE::Geometry
{
class FDynamicMesh3;
}

namespace LevelSetHelpers
{
bool CreateLevelSetForBone(UBodySetup* BodySetup, const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, uint32 InResolution);
void CreateDynamicMesh(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, UE::Geometry::FDynamicMesh3& OutMesh);
bool CreateLevelSetForMesh(const UE::Geometry::FDynamicMesh3& InMesh, int32 InLevelSetGridResolution, FKLevelSetElem& OutElement);
bool CreateLevelSetForMesh(const UE::Geometry::FDynamicMesh3& InMesh, int32 InLevelSetGridResolution, Chaos::FLevelSetPtr& OutElement);
}
