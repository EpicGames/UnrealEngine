// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetUtils.h"
#include "Modules/ModuleManager.h"
#include "MeshUtilitiesCommon.h"
#include "MeshUtilitiesEngine.h"
#include "ConvexDecompTool.h"
#include "Logging/MessageLog.h"
#include "PhysicsEngine/RigidBodyIndexPair.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/SkinnedLevelSetElem.h"
#include "Animation/SkeletalMeshActor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PreviewScene.h"
#include "Misc/ScopedSlowTask.h"
#include "SkinnedBoneTriangleCache.h"
#include "SkinnedLevelSetBuilder.h"
#include "LevelSetHelpers.h"
#include "Chaos/Levelset.h"
#include "Chaos/SkinnedTriangleMesh.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Misc/NamePermissionList.h"
#include "BoneWeights.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetUtils)

namespace FPhysicsAssetUtils
{

static const float	DefaultPrimSize = 15.0f;
static const float	MinPrimSize = 0.5f;

// This updates the collision in BodySetup associated with BoneIndex from SkelMesh. Often this is
// being done as part of a "bulk" operation, so the caller can pass in a physics asset and a list of
// bones that are going to be made physical - these can be used to help identify which bones count
// as being tied to BoneIndex, and thus which vertices etc from the skin to use when initialising
// the collision shapes.
bool CreateCollisionFromBoneInternal(
	UBodySetup* BodySetup, USkeletalMesh* SkelMesh, int32 BoneIndex, const FPhysAssetCreateParams& Params,
	const FBoneVertInfo& Info, const FSkinnedBoneTriangleCache& TriangleCache, const UPhysicsAsset* PhysicsAsset,
	TConstArrayView<int32> BoneIndicesInCreation);

// File Scope Utility Functions //
void RemoveBodyFromCollisionTableInternal(UPhysicsAsset* const PhysAsset, const int32 BodyIndex, const bool bDecrementFollowingIndices)
{
	check(PhysAsset);

	if (PhysAsset)
	{
		// First we must correct the CollisionDisableTable.
		// All elements which refer to bodyIndex are removed.
		// All elements which refer to a body with index >bodyIndex are adjusted. 

		TMap<FRigidBodyIndexPair, bool> NewCDT;
		for (int32 i = 1; i < PhysAsset->SkeletalBodySetups.Num(); i++)
		{
			for (int32 j = 0; j < i; j++)
			{
				FRigidBodyIndexPair Key(j, i);

				// If there was an entry for this pair, and it doesn't refer to the removed body, we need to add it to the new CDT.
				if (PhysAsset->CollisionDisableTable.Find(Key))
				{
					if (i != BodyIndex && j != BodyIndex)
					{
						int32 NewI = i;
						int32 NewJ = j;

						if (bDecrementFollowingIndices)
						{
							// Decrement index of all bodies with index greater than the supplied BodyIndex. This is useful when the BodyIndex is going to be removed from the physics asset.
							if (NewI > BodyIndex) { --NewI; }
							if (NewJ > BodyIndex) { --NewJ; }
						}

						FRigidBodyIndexPair NewKey(NewJ, NewI);
						NewCDT.Add(NewKey, 0);
					}
				}
			}
		}

		PhysAsset->CollisionDisableTable = NewCDT;
	}
}

void RemoveBodyConstraintsInternal(UPhysicsAsset* const PhysAsset, const int32 BodyIndex)
{
	// Now remove any constraints that were attached to this body.
	// This is a bit yuck and slow...
	TArray<int32> Constraints;
	PhysAsset->BodyFindConstraints(BodyIndex, Constraints);

	while (Constraints.Num() > 0)
	{
		DestroyConstraint(PhysAsset, Constraints[0]);
		PhysAsset->BodyFindConstraints(BodyIndex, Constraints);
	}
}

void CreateNewBodyAtIndexInternal(UPhysicsAsset* const PhysAsset, const FName InBodyName, const FPhysAssetCreateParams& Params, const int32 BodyIndex)
{
	USkeletalBodySetup* NewBodySetup = NewObject<USkeletalBodySetup>(PhysAsset, NAME_None, RF_Transactional);
	// make default to be use complex as simple 
	NewBodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;
	// newly created bodies default to simulating
	NewBodySetup->PhysicsType = PhysType_Default;

	NewBodySetup->BoneName = InBodyName;

	PhysAsset->SkeletalBodySetups[BodyIndex] = NewBodySetup;

	PhysAsset->UpdateBodySetupIndexMap();
	PhysAsset->UpdateBoundsBodiesArray();

	if (Params.bDisableCollisionsByDefault)
	{
		for (int i = 0; i < PhysAsset->SkeletalBodySetups.Num(); ++i)
		{
			PhysAsset->DisableCollision(i, BodyIndex);
		}
	}
}

// Externally Accessible Functions //

/** Returns INDEX_NONE if no children in the visual asset or if more than one parent */
static int32 GetChildIndex(int32 BoneIndex, USkeletalMesh* SkelMesh, const TArray<FBoneVertInfo>& Infos)
{
	int32 ChildIndex = INDEX_NONE;

	for(int32 i=0; i<SkelMesh->GetRefSkeleton().GetRawBoneNum(); i++)
	{
		int32 ParentIndex = SkelMesh->GetRefSkeleton().GetParentIndex(i);

		if (ParentIndex == BoneIndex && Infos[i].Positions.Num() > 0)
		{
			if(ChildIndex != INDEX_NONE)
			{
				return INDEX_NONE; // if we already have a child, this bone has more than one so return INDEX_NONE.
			}
			else
			{
				ChildIndex = i;
			}
		}
	}

	return ChildIndex;
}

static float CalcBoneInfoLength(const FBoneVertInfo& Info)
{
	FBox BoneBox(ForceInit);
	for(int32 j=0; j<Info.Positions.Num(); j++)
	{
		BoneBox += (FVector)Info.Positions[j];
	}

	if(BoneBox.IsValid)
	{
		FVector BoxExtent = BoneBox.GetExtent();
		return BoxExtent.Size();
	}
	else
	{
		return 0.f;
	}
}

/**
 * For all bones below the give bone index, find each ones minimum box dimension, and return the maximum over those bones.
 * This is used to decide if we should create physics for a bone even if its small, because there are good-sized bones below it.
 */
static float GetMaximalMinSizeBelow(int32 BoneIndex, USkeletalMesh* SkelMesh, const TArray<FBoneVertInfo>& Infos)
{
	check( Infos.Num() == SkelMesh->GetRefSkeleton().GetRawBoneNum() );

	UE_LOGF(LogPhysics, Log, "-------------------------------------------------");

	float MaximalMinBoxSize = 0.f;

	// For all bones that are children of the supplied one...
	for(int32 i=BoneIndex; i<SkelMesh->GetRefSkeleton().GetRawBoneNum(); i++)
	{
		if( SkelMesh->GetRefSkeleton().BoneIsChildOf(i, BoneIndex) )
		{
			float MinBoneDim = CalcBoneInfoLength( Infos[i] );
			
			UE_LOGF(LogPhysics, Log,  "Parent: %ls Bone: %ls Size: %f", *SkelMesh->GetRefSkeleton().GetBoneName(BoneIndex).ToString(), *SkelMesh->GetRefSkeleton().GetBoneName(i).ToString(), MinBoneDim );

			MaximalMinBoxSize = FMath::Max(MaximalMinBoxSize, MinBoneDim);
		}
	}

	return MaximalMinBoxSize;
}

void AddInfoToParentInfo(const FTransform& LocalToParentTM, const FBoneVertInfo& ChildInfo, FBoneVertInfo& ParentInfo)
{
	ParentInfo.Positions.Reserve(ParentInfo.Positions.Num() + ChildInfo.Positions.Num());
	ParentInfo.Positions.Reserve(ParentInfo.Normals.Num() + ChildInfo.Normals.Num());

	for(const FVector3f& Pos : ChildInfo.Positions)
	{
		ParentInfo.Positions.Add((FVector3f)LocalToParentTM.TransformPosition((FVector)Pos));
	}

	for (const FVector3f& Normal : ChildInfo.Normals)
	{
		ParentInfo.Normals.Add((FVector3f)LocalToParentTM.TransformVectorNoScale((FVector)Normal));
	}
}

void CreateWeightedLevelSetBody(FSkinnedLevelSetBuilder& Builder, UBodySetup* BodySetup, UPhysicsAsset* PhysAsset, FName RootBoneName, const FPhysAssetCreateParams& Params)
{
	check(BodySetup->BoneName == RootBoneName);

	BodySetup->RemoveSimpleCollision();
	BodySetup->AggGeom.SkinnedLevelSetElems.Add(Builder.CreateSkinnedLevelSetElem());
	BodySetup->InvalidatePhysicsData(); // update GUID
}

int32 CalculateCommonRootBoneIndex(const FReferenceSkeleton& RefSkeleton, const TArray<int32>& BoneIndices)
{
	// This is copied from ClothSimulationModel

	// Starts at root
	int32 RootBoneIndex = 0;

	// List of valid paths to the root bone from each bone
	TArray<TArray<int32>> PathsToRoot;
	PathsToRoot.Reserve(BoneIndices.Num());

	for (const int32 BoneIndex : BoneIndices)
	{
		TArray<int32>& Path = PathsToRoot.AddDefaulted_GetRef();

		int32 CurrentBone = BoneIndex;
		Path.Add(CurrentBone);

		while (CurrentBone != 0 && CurrentBone != INDEX_NONE)
		{
			CurrentBone = RefSkeleton.GetParentIndex(CurrentBone);
			Path.Add(CurrentBone);
		}
	}

	// Paths are from leaf->root, we want the other way
	for (TArray<int32>& Path : PathsToRoot)
	{
		Algo::Reverse(Path);
	}

	// the last common bone in all is the root
	const int32 NumPaths = PathsToRoot.Num();
	if (NumPaths > 0)
	{
		TArray<int32>& FirstPath = PathsToRoot[0];

		const int32 FirstPathSize = FirstPath.Num();
		for (int32 PathEntryIndex = 0; PathEntryIndex < FirstPathSize; ++PathEntryIndex)
		{
			const int32 CurrentQueryIndex = FirstPath[PathEntryIndex];
			bool bValidRoot = true;

			for (int32 PathIndex = 1; PathIndex < NumPaths; ++PathIndex)
			{
				if (!PathsToRoot[PathIndex].Contains(CurrentQueryIndex))
				{
					bValidRoot = false;
					break;
				}
			}

			if (bValidRoot)
			{
				RootBoneIndex = CurrentQueryIndex;
			}
			else
			{
				// Once we fail to find a valid root we're done.
				break;
			}
		}
	}
	else
	{
		// Just use the root
		RootBoneIndex = 0;
	}
	return RootBoneIndex;
}

bool CreateFromSkeletalMeshInternal(UPhysicsAsset* PhysicsAsset, USkeletalMesh* SkelMesh, const FPhysAssetCreateParams& Params, const FSkinnedBoneTriangleCache& TriangleCache, bool bShowProgress)
{
	// For each bone, get the vertices most firmly attached to it.
	TArray<FBoneVertInfo> Infos;
	FMeshUtilitiesEngine::CalcBoneVertInfos(SkelMesh, Infos, (Params.VertWeight == EVW_DominantWeight), Params.LodIndex);
	check(Infos.Num() == SkelMesh->GetRefSkeleton().GetRawBoneNum());

	PhysicsAsset->CollisionDisableTable.Empty();

	//Given the desired min body size we work from the children up to "merge" bones together. We go from leaves up because usually fingers, toes, etc... are small bones that should be merged
	//The strategy is as follows:
	//If bone is big enough, make a body
	//If not, add bone to parent for possible merge

	const TArray<FTransform> LocalPose = SkelMesh->GetRefSkeleton().GetRefBonePose();
	typedef TArray<int32> FMergedBoneIndices;

	TMap<int32, TPair<FMergedBoneIndices,FBoneVertInfo>> BoneToMergedBones;
	const int32 NumBones = Infos.Num();

	TArray<float> MergedSizes;
	MergedSizes.AddZeroed(NumBones);
	for(int32 BoneIdx = NumBones-1; BoneIdx >=0; --BoneIdx)
	{
		const float MyMergedSize = MergedSizes[BoneIdx] += CalcBoneInfoLength(Infos[BoneIdx]);

		if(MyMergedSize < Params.MinBoneSize && MyMergedSize >= Params.MinWeldSize)
		{
			//Too small to make a body for, so let's merge with parent bone. TODO: use a merge threshold
			const int32 ParentIndex = SkelMesh->GetRefSkeleton().GetParentIndex(BoneIdx);
			if(ParentIndex != INDEX_NONE)
			{
				MergedSizes[ParentIndex] += MyMergedSize;
				TPair<FMergedBoneIndices,FBoneVertInfo>& ParentMergedBones = BoneToMergedBones.FindOrAdd(ParentIndex);	//Add this bone to its parent merged bones
				ParentMergedBones.Get<0>().Add(BoneIdx);

				const FTransform LocalTM = LocalPose[BoneIdx];

				AddInfoToParentInfo(LocalTM, Infos[BoneIdx], ParentMergedBones.Get<1>());

				if(TPair<FMergedBoneIndices, FBoneVertInfo>* MyMergedBones = BoneToMergedBones.Find(BoneIdx))
				{
					//make sure any bones merged to this bone get merged into the parent
					ParentMergedBones.Get<0>().Append(MyMergedBones->Get<0>());
					AddInfoToParentInfo(LocalTM, MyMergedBones->Get<1>(), ParentMergedBones.Get<1>());

					BoneToMergedBones.Remove(BoneIdx);
				}
			}
		}
	}

	//We must ensure that there is a single root body no matter how small.
	int32 ForcedRootBoneIndex = INDEX_NONE;
	int32 FirstParentBoneIndex = INDEX_NONE;
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		if (MergedSizes[BoneIndex] > Params.MinBoneSize)
		{
			const int32 ParentBoneIndex = SkelMesh->GetRefSkeleton().GetParentIndex(BoneIndex);
			if(ParentBoneIndex == INDEX_NONE)
			{
				break;	//We already have a single root body, so don't worry about it
			}
			else if(FirstParentBoneIndex == INDEX_NONE)
			{
				FirstParentBoneIndex = ParentBoneIndex;	//record first parent to see if we have more than one root
			}
			else if(ParentBoneIndex == FirstParentBoneIndex)
			{
				ForcedRootBoneIndex = ParentBoneIndex;	//we have two "root" bodies so take their parent as the real root body
				break;
			}
		}
	}

	auto ShouldMakeBone = [&Params, &MergedSizes, ForcedRootBoneIndex](int32 BoneIndex)
	{
		// If desired - make a body for EVERY bone
		if (Params.bBodyForAll)
		{
			return true;
		}
		else if (MergedSizes[BoneIndex] > Params.MinBoneSize)
		{
			// If bone is big enough - create physics.
			return true;
		}
		else if (BoneIndex == ForcedRootBoneIndex)
		{
			// If the bone is a forced root body we must create they body no matter how small
			return true;
		}

		return false;
	};

	int32 RootBoneIndex = 0;
	if (Params.GeomType == EFG_SkinnedLevelSet)
	{
		// Calculate common root bone from all merged bones
		if (!Params.bBodyForAll)
		{
			TArray<int32> MergedBones;
			MergedBones.Reserve(NumBones);
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				if (ShouldMakeBone(BoneIndex))
				{
					MergedBones.Add(BoneIndex);
				}
			}
			RootBoneIndex = CalculateCommonRootBoneIndex(SkelMesh->GetRefSkeleton(), MergedBones);
		}
	}

	FSkinnedLevelSetBuilder LatticeBuilder(*SkelMesh, TriangleCache, RootBoneIndex);
	if (Params.GeomType == EFG_SkinnedLevelSet)
	{
		check(RootBoneIndex != INDEX_NONE);
		TArray<int32> AllBoneIndices;
		const int32 RawBoneNum = SkelMesh->GetRefSkeleton().GetRawBoneNum();
		AllBoneIndices.Reserve(RawBoneNum - RootBoneIndex);
		for (int32 BoneIndex = RootBoneIndex; BoneIndex < RawBoneNum; ++BoneIndex)
		{
			AllBoneIndices.Emplace(BoneIndex);
		}
		TArray<uint32> OrigIndicesUnused;
		if (!LatticeBuilder.InitializeSkinnedLevelset(Params, AllBoneIndices, OrigIndicesUnused))
		{
			return false;
		}
	}

	// Finally, iterate through all the bones and create bodies when needed

	const bool bCanCreateConstraints = CanCreateConstraints();
	FScopedSlowTask SlowTask((float)NumBones * 2, FText(), bShowProgress&& IsInGameThread());
	if (bShowProgress && IsInGameThread())
	{
		SlowTask.MakeDialog();
	}

	// When creating bodies, sometimes we will need to find other non-physical (helper) bones to
	// include. For this, we need to know which physical bones to exclude.
	TArray<int32> PhysicalBoneIndices;
	for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		// Determine if we should create a physics body for this bone
		if (ShouldMakeBone(BoneIndex))
		{
			PhysicalBoneIndices.Emplace(BoneIndex);
		}
	}

	for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		// Determine if we should create a physics body for this bone
		const bool bMakeBone = ShouldMakeBone(BoneIndex);

		if (bMakeBone)
		{
			// Go ahead and make this bone physical.
			FName BoneName = SkelMesh->GetRefSkeleton().GetBoneName(BoneIndex);

			if (bShowProgress && IsInGameThread())
			{
				SlowTask.EnterProgressFrame(1.0f, FText::Format(NSLOCTEXT(
					"PhysicsAssetEditor", "ResetCollsionStepInfo", "Generating collision for {0}"), FText::FromName(BoneName)));
			}

			//Construct the info - in the case of merged bones we append all the data
			FBoneVertInfo Info = Infos[BoneIndex];
			FMergedBoneIndices BoneIndices;
			BoneIndices.Add(BoneIndex);
			if (const TPair<FMergedBoneIndices, FBoneVertInfo>* MergedBones = BoneToMergedBones.Find(BoneIndex))
			{
				//Don't need to convert into parent space since this was already done
				BoneIndices.Append(MergedBones->Get<0>());
				Info.Normals.Append(MergedBones->Get<1>().Normals);
				Info.Positions.Append(MergedBones->Get<1>().Positions);
			}

			if (Params.GeomType == EFG_SkinnedLevelSet)
			{
				LatticeBuilder.AddBoneInfluence(BoneIndex, BoneIndices);
			}
			else
			{
				const int32 NewBodyIndex = CreateNewBody(PhysicsAsset, BoneName, Params);
				UBodySetup* NewBodySetup = PhysicsAsset->SkeletalBodySetups[NewBodyIndex];
				check(NewBodySetup->BoneName == BoneName);

				// Fill in collision info for this bone.
				const bool bSuccess = CreateCollisionFromBoneInternal(
					NewBodySetup, SkelMesh, BoneIndex, Params, Info, TriangleCache, PhysicsAsset, PhysicalBoneIndices);
				if (bSuccess)
				{
					// create joint to parent body
					if (Params.bCreateConstraints && bCanCreateConstraints)
					{
						int32 ParentIndex = BoneIndex;
						int32 ParentBodyIndex = INDEX_NONE;
						FName ParentName;

						do
						{
							//Travel up the hierarchy to find a parent which has a valid body
							ParentIndex = SkelMesh->GetRefSkeleton().GetParentIndex(ParentIndex);
							if (ParentIndex != INDEX_NONE)
							{
								ParentName = SkelMesh->GetRefSkeleton().GetBoneName(ParentIndex);
								ParentBodyIndex = PhysicsAsset->FindBodyIndex(ParentName);
							}
							else
							{
								//no more parents so just stop
								break;
							}

						} while (ParentBodyIndex == INDEX_NONE);

						if (ParentBodyIndex != INDEX_NONE)
						{
							//found valid parent body so create joint
							int32 NewConstraintIndex = CreateNewConstraint(PhysicsAsset, BoneName);
							UPhysicsConstraintTemplate* CS = PhysicsAsset->ConstraintSetup[NewConstraintIndex];

							// set angular constraint mode
							CS->DefaultInstance.SetAngularSwing1Motion(Params.AngularConstraintMode);
							CS->DefaultInstance.SetAngularSwing2Motion(Params.AngularConstraintMode);
							CS->DefaultInstance.SetAngularTwistMotion(Params.AngularConstraintMode);

							// Place joint at origin of child
							CS->DefaultInstance.ConstraintBone1 = BoneName;
							CS->DefaultInstance.ConstraintBone2 = ParentName;
							CS->DefaultInstance.SnapTransformsToDefault(EConstraintTransformComponentFlags::All, PhysicsAsset);

							CS->SetDefaultProfile(CS->DefaultInstance);

							// Disable collision between constrained bodies by default.
							PhysicsAsset->DisableCollision(NewBodyIndex, ParentBodyIndex);
						}
					}
				}
				else
				{
					DestroyBody(PhysicsAsset, NewBodyIndex);
				}
			}
		}
	}

	if (Params.GeomType == EFG_SkinnedLevelSet)
	{
		// Finish Building WeightedLattice
		const FName RootBoneName = SkelMesh->GetRefSkeleton().GetBoneName(RootBoneIndex);
		const int32 NewBodyIndex = CreateNewBody(PhysicsAsset, RootBoneName, Params);
		UBodySetup* NewBodySetup = PhysicsAsset->SkeletalBodySetups[NewBodyIndex];
		CreateWeightedLevelSetBody(LatticeBuilder, NewBodySetup, PhysicsAsset, RootBoneName, Params);
	}

	//Go through and ensure any overlapping bodies are marked as disable collision
	FPreviewScene TmpScene;
	UWorld* TmpWorld = TmpScene.GetWorld();
	ASkeletalMeshActor* SkeletalMeshActor = TmpWorld->SpawnActor<ASkeletalMeshActor>(ASkeletalMeshActor::StaticClass(), FTransform::Identity);
	SkeletalMeshActor->GetSkeletalMeshComponent()->SetSkeletalMesh(SkelMesh);
	USkeletalMeshComponent* SKC = SkeletalMeshActor->GetSkeletalMeshComponent();
	SKC->SetPhysicsAsset(PhysicsAsset);
	SkeletalMeshActor->RegisterAllComponents();
	

	const TArray<FBodyInstance*> Bodies = SKC->Bodies;
	const int32 NumBodies = Bodies.Num();
	for(int32 BodyIdx = 0; BodyIdx < NumBodies; ++BodyIdx)
	{
		FBodyInstance* BodyInstance = Bodies[BodyIdx];
		if(BodyInstance && BodyInstance->BodySetup.IsValid())
		{
			if (bShowProgress && IsInGameThread())
			{
				SlowTask.EnterProgressFrame(1.0f, FText::Format(NSLOCTEXT("PhysicsAssetEditor", "ResetCollsionStepInfoOverlaps", "Fixing overlaps for {0}"), FText::FromName(BodyInstance->BodySetup->BoneName)));
			}

			FTransform BodyTM = BodyInstance->GetUnrealWorldTransform();

			for(int32 OtherBodyIdx = BodyIdx + 1; OtherBodyIdx < NumBodies; ++OtherBodyIdx)
			{
				FBodyInstance* OtherBodyInstance = Bodies[OtherBodyIdx];
				if(OtherBodyInstance && OtherBodyInstance->BodySetup.IsValid())
				{
					if(BodyInstance->OverlapTestForBody(BodyTM.GetLocation(), BodyTM.GetRotation(), OtherBodyInstance))
					{
						PhysicsAsset->DisableCollision(BodyIdx, OtherBodyIdx);
					}
				}
			}
		}
	}

	return NumBodies > 0;
}

bool CreateFromSkeletalMesh(UPhysicsAsset* PhysicsAsset, USkeletalMesh* SkelMesh, const FPhysAssetCreateParams& Params, FText& OutErrorMessage, bool bSetToMesh /*= true*/, bool bShowProgress /*= true */)
{
	PhysicsAsset->PreviewSkeletalMesh = SkelMesh;

	check(SkelMesh);
	check(SkelMesh->GetResourceForRendering());

	FSkinnedBoneTriangleCache TriangleCache(*SkelMesh, Params);

	if (Params.bAlwaysUseVertices || Params.GeomType == EFG_SingleConvexHull ||
		Params.GeomType == EFG_MultiConvexHull || Params.GeomType == EFG_LevelSet ||
		Params.GeomType == EFG_SkinnedLevelSet)
	{
		TriangleCache.BuildCache();
	}

	bool bSuccess = CreateFromSkeletalMeshInternal(PhysicsAsset, SkelMesh, Params, TriangleCache, bShowProgress);
	if (!bSuccess)
	{
		// try lower minimum bone size 
		FPhysAssetCreateParams LocalParams = Params;
		LocalParams.MinBoneSize = 1.f;

		bSuccess = CreateFromSkeletalMeshInternal(PhysicsAsset, SkelMesh, LocalParams, TriangleCache, bShowProgress);

		if(!bSuccess)
		{
			OutErrorMessage = FText::Format(NSLOCTEXT("CreatePhysicsAsset", "CreatePhysicsAssetLinkFailed", "The bone size is too small to create Physics Asset '{0}' from Skeletal Mesh '{1}'. You will have to create physics asset manually."), FText::FromString(PhysicsAsset->GetName()), FText::FromString(SkelMesh->GetName()));
		}
	}

	if(bSuccess && bSetToMesh)
	{
		SkelMesh->SetPhysicsAsset(PhysicsAsset);
		SkelMesh->MarkPackageDirty();
	}

	return bSuccess;
}

template<typename RealType>
FMatrix ComputeViewCovarianceMatrix(TArrayView<const UE::Math::TVector<RealType>, int> PointView)
{
	using FVectorType = UE::Math::TVector<RealType>;

	if(PointView.Num() == 0)
	{
		return FMatrix::Identity;
	}

	//get average
	const RealType N = PointView.Num();
	FVectorType U = FVectorType::ZeroVector;
	
	for(const FVectorType& Point : PointView)
	{
		U += Point;
	}
	
	U = U / N;

	//compute error terms
	TArray<FVectorType> Errors;
	Errors.AddUninitialized(N);

	for(int32 i = 0; i < N; ++i)
	{
		Errors[i] = PointView[i] - U;
	}

	FMatrix Covariance = FMatrix::Identity;
	for(int32 j = 0; j < 3; ++j)
	{
		FVectorType Axis = FVectorType::ZeroVector;
		RealType* Cj = &Axis.X;
		for(int32 k = 0; k < 3; ++k)
		{
			RealType Cjk = 0.f;
			for(int32 i = 0; i < N; ++i)
			{
				const RealType* Error = &Errors[i].X;
				Cj[k] += Error[j] * Error[k];
			}
			Cj[k] /= N;
		}

		Covariance.SetAxis(j, UE::Math::TVector<double>(Axis));
	}

	return Covariance;
}

FMatrix ComputeCovarianceMatrix(const TArray<FVector3f>& Verts)
{
	return ComputeViewCovarianceMatrix(MakeArrayView(Verts));
}

FVector ComputeEigenVector(const FMatrix& A)
{
	// Using the power method: this is ok because we only need the dominant eigenvector and speed is
	// not critical: http://en.wikipedia.org/wiki/Power_iteration
	FVector Bk = FVector(0, 0, 1);
	for (int32 i = 0; i < 32; ++i)
	{
		float Length = Bk.Size();
		if (Length > 0.f)
		{
			Bk = A.TransformVector(Bk) / Length;
		}
	}

	return Bk.GetSafeNormal();
}

/**
 * Collects BoneIndex plus all descendant bone indices in the skeleton hierarchy, stopping at any 
 * bone that has its own physics body in the PhysicsAsset.
 * 
 * Note that this may return an empty array (OutBoneIndices) if the skeletal mesh, physics asset, 
 * reference skeleton etc turn out to be invalid/problematic.
 */
static void GetRelevantBoneIndices(
	const USkeletalMesh*   SkeletalMesh, 
	const UPhysicsAsset*   PhysicsAsset, 
	const int32            BoneIndex, 
	TArray<int32>&         OutBoneIndices,
	TConstArrayView<int32> BoneIndicesInCreation)
{
	if (!SkeletalMesh || !PhysicsAsset || BoneIndex == INDEX_NONE)
	{
		return;
	}

	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const int32 NumBones = RefSkeleton.GetNum();

	if (BoneIndex < 0 || BoneIndex >= NumBones)
	{
		return;
	}

	// Build a set of bone indices that have their own physics body in the asset.
	// We look up by bone name since that's the reliable link between skeleton and physics asset.
	TSet<int32> PhysicalBoneIndices;
	for (const USkeletalBodySetup* BodySetup : PhysicsAsset->SkeletalBodySetups)
	{
		if (BodySetup)
		{
			int32 BodyBoneIdx = RefSkeleton.FindBoneIndex(BodySetup->BoneName);
			if (BodyBoneIdx != INDEX_NONE)
			{
				PhysicalBoneIndices.Add(BodyBoneIdx);
			}
		}
	}

	// Duplicates are automatically ignored
	PhysicalBoneIndices.Append(BoneIndicesInCreation);

	// Iterative traversal down the hierarchy. Always include the starting bone (it's the one we're
	// generating the hull for). For children, include them only if they are NOT physical, then
	// continue recursing.
	OutBoneIndices.Reset();
	TArray<int32> Stack;
	Stack.Push(BoneIndex);

	while (Stack.Num() > 0)
	{
		const int32 Current = Stack.Pop(EAllowShrinking::No);
		OutBoneIndices.Add(Current);

		// Find all direct children of Current in the ref skeleton.
		for (int32 ChildIdx = 0; ChildIdx < NumBones; ++ChildIdx)
		{
			if (RefSkeleton.GetParentIndex(ChildIdx) == Current)
			{
				// Stop at (don't include) bones that have their own physics body,
				// UNLESS it's the starting bone itself (which is physical by definition).
				if (!PhysicalBoneIndices.Contains(ChildIdx))
				{
					Stack.Push(ChildIdx);
				}
			}
		}
	}
}

bool CreateCollisionFromBoneInternal(
	UBodySetup* BodySetup, USkeletalMesh* SkelMesh, int32 BoneIndex, const FPhysAssetCreateParams& Params, 
	const FBoneVertInfo& Info, const FSkinnedBoneTriangleCache& TriangleCache, const UPhysicsAsset* PhysicsAsset,
	TConstArrayView<int32> BoneIndicesInCreation)
{
	// Calculate orientation of to use for collision primitive.
	FMatrix ElemTM = FMatrix::Identity;

	const TArray<FVector3f>* VertsToUse = &Info.Positions;
	TArray<FVector3f> TriangleCacheVerts;
	TArray<uint32> TriangleCacheIndices;

	if (TriangleCache.IsBuilt())
	{
		TArray<int32> BoneIndices;
		if (Params.bIncludeChildBones)
		{
			GetRelevantBoneIndices(SkelMesh, PhysicsAsset, BoneIndex, BoneIndices, BoneIndicesInCreation);
		}
		else
		{
			BoneIndices.Add(BoneIndex);
		}

		TArray<uint32> Unused;
		TriangleCache.GetVerticesAndIndicesForBones(BoneIndex, BoneIndices, TriangleCacheVerts, TriangleCacheIndices, Unused);

		if (TriangleCacheVerts.IsEmpty())
		{
			if (BodySetup->AggGeom.GetElementCount() > 0)
			{
				// If we're supposed to be using the mesh but have failed to find any verts, then
				// just leave any existing collision as it is.
				FMessageLog EditorErrors("EditorErrors");
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("BoneName"), FText::FromName(SkelMesh->GetRefSkeleton().GetBoneName(BoneIndex)));
				EditorErrors.Warning(FText::Format(NSLOCTEXT("PhysicsAssetUtils", "NoVertsUpdateHull2",
					"Unable to update hull for {BoneName} as there are no vertices associated with the bone."), Arguments));
				EditorErrors.Open();
				return false;
			}
		}
		else
		{
			VertsToUse = &TriangleCacheVerts;
		}
	}

	// Convex hulls aren't user-editable after creation, so don't set them unless we know the result
	// will be good. Other shapes can be tweaked by the user, so even if we don't have vertices,
	// create/change to them as the user can fix them up.
	if (Params.GeomType == EFG_SingleConvexHull || Params.GeomType == EFG_MultiConvexHull)
	{
		if (VertsToUse->IsEmpty())
		{
			// If there are no verts, then either leave the shape as is, or create something "default"
			FMessageLog EditorErrors("EditorErrors");
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("BoneName"), FText::FromName(SkelMesh->GetRefSkeleton().GetBoneName(BoneIndex)));
			if (BodySetup->AggGeom.GetElementCount() > 0)
			{
				EditorErrors.Warning(FText::Format(NSLOCTEXT("PhysicsAssetUtils", "NoVertsUpdateHull1",
					"Unable to update hull for {BoneName} as there are no vertices associated with the bone."), Arguments));
				EditorErrors.Open();
				return false;
			}
			EditorErrors.Warning(FText::Format(NSLOCTEXT("PhysicsAssetUtils", "NoVertsCreateShape",
				"Unable to create a good shape for the {BoneName} as there are no vertices associated with it."), Arguments));
			EditorErrors.Open();
		}
	}

	if (Params.bAutoOrientToBone)
	{
		const FMatrix CovarianceMatrix = ComputeCovarianceMatrix(*VertsToUse);
		FVector ZAxis = ComputeEigenVector(CovarianceMatrix);
		FVector XAxis, YAxis;
		ZAxis.FindBestAxisVectors(YAxis, XAxis);
		ElemTM = FMatrix(XAxis, YAxis, ZAxis, FVector::ZeroVector);
	}

	// Convert to ElemTM to FTransform as Matrix inverse doesn't handle well when DET == 0, so 
	FTransform ElementTransform(ElemTM);
	// Get the (Unreal scale) bounding box for this bone using the rotation.
	FBox BoneBox(ForceInit);
	for (const FVector3f& Vert : *VertsToUse)
	{
		BoneBox += ElementTransform.InverseTransformPosition(FVector(Vert));
	}

	// Verts are already in the bone's space (including scale)
	FVector BoxCenter(0, 0, 0), BoxExtent(0, 0, 0);
	if (BoneBox.IsValid)
	{
		BoneBox.GetCenterAndExtents(BoxCenter, BoxExtent);
	}
	BoxExtent = FVector::Max(BoxExtent, FVector(MinPrimSize, MinPrimSize, MinPrimSize));

	FVector BoneOrigin = ElementTransform.TransformPosition(BoxCenter);
	ElementTransform.SetTranslation(BoneOrigin);

	if (Params.GeomType == EFG_Box)
	{
		// Add a new box geometry to this body the size of the bounding box.
		FKBoxElem BoxElem;

		BoxElem.SetTransform(ElementTransform);

		BoxElem.X = BoxExtent.X * 2.0f * 1.01f; // Side Lengths (add 1% to avoid graphics glitches)
		BoxElem.Y = BoxExtent.Y * 2.0f * 1.01f;
		BoxElem.Z = BoxExtent.Z * 2.0f * 1.01f;

		BodySetup->RemoveSimpleCollision();
		BodySetup->AggGeom.BoxElems.Add(BoxElem);
		BodySetup->InvalidatePhysicsData();
	}
	else if (Params.GeomType == EFG_Sphere)
	{
		FKSphereElem SphereElem;

		SphereElem.Center = ElementTransform.GetTranslation();
		SphereElem.Radius = BoxExtent.GetMax() * 1.01f;

		BodySetup->RemoveSimpleCollision();
		BodySetup->AggGeom.SphereElems.Add(SphereElem);
		BodySetup->InvalidatePhysicsData();
	}
	else if (Params.GeomType == EFG_SingleConvexHull || Params.GeomType == EFG_MultiConvexHull)
	{
		// We need to pick up vertices from this bone and any bone below it in the hierarchy,
		// stopping the search when we move on to another bone with physics.
		if (!TriangleCacheIndices.IsEmpty())
		{
			const int32 HullCount = Params.GeomType == EFG_SingleConvexHull ? 1 : Params.HullCount;
			DecomposeMeshToHulls(BodySetup, *VertsToUse, TriangleCacheIndices, HullCount, Params.MaxHullVerts);
		}
		else
		{
			FMessageLog EditorErrors("EditorErrors");
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("BoneName"), FText::FromName(SkelMesh->GetRefSkeleton().GetBoneName(BoneIndex)));
			EditorErrors.Warning(FText::Format(NSLOCTEXT("PhysicsAssetUtils", "ConvexNoPositions",
				"Unable to create a convex hull for {BoneName} as there are no vertices associated with the bone."), Arguments));
			EditorErrors.Open();
			return false;
		}
	}
	else if (Params.GeomType == EFG_Sphyl)
	{
		FKSphylElem SphylElem;

		if (BoxExtent.X > BoxExtent.Z && BoxExtent.X > BoxExtent.Y)
		{
			//X is the biggest so we must rotate X-axis into Z-axis
			SphylElem.SetTransform(FTransform(FQuat(FVector(0, 1, 0), -PI * 0.5f)) * ElementTransform);
			SphylElem.Radius = FMath::Max(BoxExtent.Y, BoxExtent.Z) * 1.01f;
			SphylElem.Length = BoxExtent.X * 1.01f;

		}
		else if (BoxExtent.Y > BoxExtent.Z && BoxExtent.Y > BoxExtent.X)
		{
			//Y is the biggest so we must rotate Y-axis into Z-axis
			SphylElem.SetTransform(FTransform(FQuat(FVector(1, 0, 0), PI * 0.5f)) * ElementTransform);
			SphylElem.Radius = FMath::Max(BoxExtent.X, BoxExtent.Z) * 1.01f;
			SphylElem.Length = BoxExtent.Y * 1.01f;
		}
		else
		{
			//Z is the biggest so use transform as is
			SphylElem.SetTransform(ElementTransform);

			SphylElem.Radius = FMath::Max(BoxExtent.X, BoxExtent.Y) * 1.01f;
			SphylElem.Length = BoxExtent.Z * 1.01f;
		}

		BodySetup->RemoveSimpleCollision();
		BodySetup->AggGeom.SphylElems.Add(SphylElem);
		BodySetup->InvalidatePhysicsData();
	}
	else if (Params.GeomType == EFG_TaperedCapsule)
	{
		FKTaperedCapsuleElem TaperedCapsuleElem;

		if (BoxExtent.X > BoxExtent.Z && BoxExtent.X > BoxExtent.Y)
		{
			//X is the biggest so we must rotate X-axis into Z-axis
			TaperedCapsuleElem.SetTransform(FTransform(FQuat(FVector(0, 1, 0), -PI * 0.5f)) * ElementTransform);
			TaperedCapsuleElem.Radius0 = FMath::Max(BoxExtent.Y, BoxExtent.Z) * 1.01f;
			TaperedCapsuleElem.Radius1 = FMath::Max(BoxExtent.Y, BoxExtent.Z) * 1.01f;
			TaperedCapsuleElem.Length = BoxExtent.X * 1.01f;
		}
		else if (BoxExtent.Y > BoxExtent.Z && BoxExtent.Y > BoxExtent.X)
		{
			//Y is the biggest so we must rotate Y-axis into Z-axis
			TaperedCapsuleElem.SetTransform(FTransform(FQuat(FVector(1, 0, 0), PI * 0.5f)) * ElementTransform);
			TaperedCapsuleElem.Radius0 = FMath::Max(BoxExtent.X, BoxExtent.Z) * 1.01f;
			TaperedCapsuleElem.Radius1 = FMath::Max(BoxExtent.X, BoxExtent.Z) * 1.01f;
			TaperedCapsuleElem.Length = BoxExtent.Y * 1.01f;
		}
		else
		{
			//Z is the biggest so use transform as is
			TaperedCapsuleElem.SetTransform(ElementTransform);
			TaperedCapsuleElem.Radius0 = FMath::Max(BoxExtent.X, BoxExtent.Y) * 1.01f;
			TaperedCapsuleElem.Radius1 = FMath::Max(BoxExtent.X, BoxExtent.Y) * 1.01f;
			TaperedCapsuleElem.Length = BoxExtent.Z * 1.01f;
		}

		BodySetup->RemoveSimpleCollision();
		BodySetup->AggGeom.TaperedCapsuleElems.Add(TaperedCapsuleElem);
		BodySetup->InvalidatePhysicsData();
	}
	else if (Params.GeomType == EFG_LevelSet)
	{
		if (!VertsToUse->IsEmpty() && !TriangleCacheIndices.IsEmpty())
		{
			// This clears out existing collision (or, if it fails, it leaves it)
			const bool bOK = LevelSetHelpers::CreateLevelSetForBone(BodySetup, *VertsToUse, TriangleCacheIndices, Params.LevelSetResolution);
			if (!bOK)
			{
				FMessageLog EditorErrors("EditorErrors");
				EditorErrors.Warning(NSLOCTEXT("PhysicsAssetUtils", "LevelSetError", "An error occurred creating a level set for the given bone."));
				EditorErrors.Open();
				return false;
			}
		}
		else
		{
			FMessageLog EditorErrors("EditorErrors");
			EditorErrors.Warning(NSLOCTEXT("PhysicsAssetUtils", "LevelSetNoPositions", "Unable to create a level set for the given bone as there are no vertices associated with the bone."));
			EditorErrors.Open();
			return false;
		}
	}

	return true;
}

bool CreateCollisionFromBone(
	UBodySetup* BodySetup, USkeletalMesh* SkelMesh, int32 BoneIndex, const FPhysAssetCreateParams& Params, 
	const FBoneVertInfo& Info, const UPhysicsAsset* PhysicsAsset, TConstArrayView<int32> BoneIndicesInCreation)
{
	check(SkelMesh);

	FSkinnedBoneTriangleCache TriangleCache(*SkelMesh, Params);

	if (Params.bAlwaysUseVertices || Params.GeomType == EFG_SingleConvexHull ||
		Params.GeomType == EFG_MultiConvexHull || Params.GeomType == EFG_LevelSet ||
		Params.GeomType == EFG_SkinnedLevelSet)
	{
		TriangleCache.BuildCache();
	}

	return CreateCollisionFromBoneInternal(
		BodySetup, SkelMesh, BoneIndex, Params, Info, TriangleCache, PhysicsAsset, BoneIndicesInCreation);
}

bool CreateCollisionFromBones(
	UBodySetup* BodySetup, USkeletalMesh* SkelMesh, const TArray<int32>& BoneIndices, const FPhysAssetCreateParams& Params, 
	const FBoneVertInfo& Info, const UPhysicsAsset* PhysicsAsset)
{
	check(SkelMesh);

	FSkinnedBoneTriangleCache TriangleCache(*SkelMesh, Params);

	if (Params.bAlwaysUseVertices || Params.GeomType == EFG_SingleConvexHull ||
		Params.GeomType == EFG_MultiConvexHull || Params.GeomType == EFG_LevelSet ||
		Params.GeomType == EFG_SkinnedLevelSet)
	{
		TriangleCache.BuildCache();
	}

	bool bAllSuccessful = true;
	for ( int Index = 0; Index < BoneIndices.Num(); ++Index )
	{
		if ( !CreateCollisionFromBoneInternal(
			BodySetup, SkelMesh, BoneIndices[Index], Params, Info, TriangleCache, PhysicsAsset, BoneIndices) )
		{
			bAllSuccessful = false;
		}
	}

	return bAllSuccessful;
}

bool CreateCollisionsFromBones(
	UPhysicsAsset* PhysicsAsset, USkeletalMesh* SkelMesh, const TArray<int32>& BodyIndices, 
	const FPhysAssetCreateParams& Params, const TArray<FBoneVertInfo>& Info, TArray<int32>& OutSuccessfulBodyIndices)
{
	check(SkelMesh);
	check(PhysicsAsset);

	FSkinnedBoneTriangleCache TriangleCache(*SkelMesh, Params);

	if (Params.bAlwaysUseVertices || Params.GeomType == EFG_SingleConvexHull || 
		Params.GeomType == EFG_MultiConvexHull || Params.GeomType == EFG_LevelSet || 
		Params.GeomType == EFG_SkinnedLevelSet)
	{
		TriangleCache.BuildCache();
	}

	OutSuccessfulBodyIndices.Reset();
	bool bAllSuccessful = true;

	// Destroying bodies causes the indices to change. Use names instead.
	TArray<FName> BodySetupBoneNames;
	BodySetupBoneNames.Reserve(BodyIndices.Num());
	TArray<FName> SubBoneNames;
	for (int32 Index = 0; Index < BodyIndices.Num(); ++Index)
	{
		const UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndices[Index]];
		BodySetupBoneNames.Add(BodySetup->BoneName);

		// Get sub-bones for any skinned levelsets
		for (const FKSkinnedLevelSetElem& SkinnedLevelSetElem : BodySetup->AggGeom.SkinnedLevelSetElems)
		{
			if (const Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>* SkinnedLevelSet = SkinnedLevelSetElem.WeightedLevelSet().GetReference())
			{
				SubBoneNames.Reserve(SubBoneNames.Num() + SkinnedLevelSet->GetUsedBones().Num());
				for (const FName& SubBoneName : SkinnedLevelSet->GetUsedBones())
				{
					SubBoneNames.AddUnique(SubBoneName);
				}
			}
		}
	}

	int32 RootBoneIndex = INDEX_NONE;
	TArray<int32> AllBoneIndices;
	if (Params.GeomType == EFG_SkinnedLevelSet)
	{
		AllBoneIndices.Reserve(BodySetupBoneNames.Num());
		for (int32 Index = 0; Index < BodySetupBoneNames.Num(); ++Index)
		{
			const int32 BoneIndex = SkelMesh->GetRefSkeleton().FindBoneIndex(BodySetupBoneNames[Index]);
			AllBoneIndices.Add(BoneIndex);
		}
		RootBoneIndex = CalculateCommonRootBoneIndex(SkelMesh->GetRefSkeleton(), AllBoneIndices);
	}

	FSkinnedLevelSetBuilder LatticeBuilder(*SkelMesh, TriangleCache, RootBoneIndex);
	if (Params.GeomType == EFG_SkinnedLevelSet)
	{
		TArray<uint32> OrigIndices;
		if (!LatticeBuilder.InitializeSkinnedLevelset(Params, AllBoneIndices, OrigIndices))
		{
			return false;
		}

		if (Params.VertWeight == EVW_AnyWeight)
		{
			// Add additional sub-bones that contribute to the vertices
			TSet<int32> SubBoneSet;
			LatticeBuilder.GetInfluencingBones(OrigIndices, SubBoneSet);

			for (int32 AddedSubBone : SubBoneSet)
			{
				SubBoneNames.AddUnique(SkelMesh->GetRefSkeleton().GetBoneName(AddedSubBone));
			}
		}
	}

	FScopedSlowTask SlowTask((float)BodySetupBoneNames.Num() + (float)SubBoneNames.Num());
	if (IsInGameThread())
	{
		SlowTask.MakeDialog();
	}

	TArray<FName> SuccessfulBodySetupNames;
	for (const FName& BoneName : BodySetupBoneNames)
	{
		if (IsInGameThread())
		{
			SlowTask.EnterProgressFrame(1.0f, FText::Format(NSLOCTEXT("PhysicsAssetEditor", "ResetCollsionStepInfo", "Generating collision for {0}"), FText::FromName(BoneName)));
		}

		const int32 BodyIndex = PhysicsAsset->FindBodyIndex(BoneName);
		check(BodyIndex != INDEX_NONE);

		UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
		BodySetup->Modify();
		check(BodySetup);
		const int32 BoneIndex = SkelMesh->GetRefSkeleton().FindBoneIndex(BodySetup->BoneName);
		check(BoneIndex != INDEX_NONE);

		if (Params.GeomType == EFG_SkinnedLevelSet)
		{
			if (BoneIndex != RootBoneIndex)
			{
				// This bone is now getting merged into the RootBoneIndex's body
				DestroyBody(PhysicsAsset, BodyIndex);
			}

			TArray<int32> BoneIndexArray;
			BoneIndexArray.Add(BoneIndex);
			LatticeBuilder.AddBoneInfluence(BoneIndex, BoneIndexArray);
		}
		else
		{
			TArray<int32> BoneIndicesInCreation;
			BoneIndicesInCreation.Reserve(BodyIndices.Num());
			for (int32 Index = 0; Index < BodySetupBoneNames.Num(); ++Index)
			{
				const int32 BoneIndexInCreation = SkelMesh->GetRefSkeleton().FindBoneIndex(BodySetupBoneNames[Index]);
				BoneIndicesInCreation.Add(BoneIndexInCreation);
			}

			if (CreateCollisionFromBoneInternal(
				BodySetup, SkelMesh, BoneIndex, Params, Info[BoneIndex], TriangleCache, PhysicsAsset, BoneIndicesInCreation))
			{
				SuccessfulBodySetupNames.AddUnique(BoneName);
			}
			else
			{
				bAllSuccessful = false;

				// If collision generation failed, then there are two possibilities:
				// 1. It was a fresh generation, in which case we have a body with no shapes, which is Bad and must go.
				// 2. It was a regeneration, in which case we have the unchanged shapes from before - keep them.
				if (BodySetup->AggGeom.GetElementCount() == 0)
				{
					DestroyBody(PhysicsAsset, BodyIndex);
				}
			}
		}
	}

	for (const FName& BoneName : SubBoneNames)
	{
		if (IsInGameThread())
		{
			SlowTask.EnterProgressFrame(1.0f, FText::Format(NSLOCTEXT("PhysicsAssetEditor", "ResetCollsionStepInfo", "Generating collision for {0}"), FText::FromName(BoneName)));
		}

		if (BodySetupBoneNames.Find(BoneName) != INDEX_NONE)
		{
			// We've already dealt with this above.
			continue;
		}

		if (Params.GeomType == EFG_SkinnedLevelSet)
		{
			const int32 BoneIndex = SkelMesh->GetRefSkeleton().FindBoneIndex(BoneName);
			TArray<int32> BoneIndexArray;
			BoneIndexArray.Add(BoneIndex);
			LatticeBuilder.AddBoneInfluence(BoneIndex, BoneIndexArray);
		}
		else
		{
			if (PhysicsAsset->FindBodyIndex(BoneName) != INDEX_NONE)
			{
				// We already have an existing bone. Don't replace it.
				continue;
			}
			const int32 NewBodyIndex = CreateNewBody(PhysicsAsset, BoneName, Params);
			UBodySetup* NewBodySetup = PhysicsAsset->SkeletalBodySetups[NewBodyIndex];
			check(NewBodySetup->BoneName == BoneName);
			const int32 BoneIndex = SkelMesh->GetRefSkeleton().FindBoneIndex(NewBodySetup->BoneName);


			TArray<int32> BoneIndicesInCreation;
			BoneIndicesInCreation.Reserve(BodyIndices.Num());
			for (int32 Index = 0; Index < BodySetupBoneNames.Num(); ++Index)
			{
				const int32 BoneIndexInCreation = SkelMesh->GetRefSkeleton().FindBoneIndex(BodySetupBoneNames[Index]);
				BoneIndicesInCreation.Add(BoneIndexInCreation);
			}

			if (CreateCollisionFromBoneInternal(
				NewBodySetup, SkelMesh, BoneIndex, Params, Info[BoneIndex], TriangleCache, PhysicsAsset, BoneIndicesInCreation))
			{
				SuccessfulBodySetupNames.AddUnique(BoneName);
			}
			else
			{
				bAllSuccessful = false;
				if (NewBodySetup->AggGeom.GetElementCount() == 0)
				{
					DestroyBody(PhysicsAsset, NewBodyIndex);
				}
			}
		}
	}

	OutSuccessfulBodyIndices.Reserve(SuccessfulBodySetupNames.Num());
	for (const FName& BoneName : SuccessfulBodySetupNames)
	{
		const int32 BodyIndex = PhysicsAsset->FindBodyIndex(BoneName);
		check(BodyIndex != INDEX_NONE);
		OutSuccessfulBodyIndices.Add(BodyIndex);
	}

	if (Params.GeomType == EFG_SkinnedLevelSet)
	{
		// Finish Building WeightedLattice
		const FName RootBoneName = SkelMesh->GetRefSkeleton().GetBoneName(RootBoneIndex);
		const int32 BodyIndex = CreateNewBody(PhysicsAsset, RootBoneName, Params);
		UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
		CreateWeightedLevelSetBody(LatticeBuilder, BodySetup, PhysicsAsset, RootBoneName, Params);
		OutSuccessfulBodyIndices.Add(BodyIndex);
	}
	return bAllSuccessful;
}

int32 GenerateSkinnedTriangleMesh(UPhysicsAsset* PhysicsAsset, USkeletalMesh* SkeletalMesh, int32 LODIndex, const FPhysAssetCreateParams& Params, FText& OutErrorMessage, bool bShowProgress)
{
	if (!ensure(PhysicsAsset))
	{
		OutErrorMessage = FText(NSLOCTEXT("GenerateSkinnedTriangleMesh", "MissingPAFailure", "Failed to generate the skinned triangle mesh: missing input PhysicsAsset"));
		return INDEX_NONE;
	}
	if (!ensure(SkeletalMesh))
	{
		OutErrorMessage = FText(NSLOCTEXT("GenerateSkinnedTriangleMesh", "MissingSKMFailure", "Failed to generate the skinned triangle mesh: missing input Skeletal Mesh"));
		return INDEX_NONE;
	}

	// Gather data from the SkelMesh.
	const FSkeletalMeshRenderData* const RenderData = SkeletalMesh->GetResourceForRendering();
	if (!ensure(RenderData))
	{
		OutErrorMessage = FText::Format(NSLOCTEXT("GenerateSkinnedTriangleMesh", "MissingRenderDataFailure", "Failed to generate the skinned triangle mesh: missing Render Data on Skeletal Mesh '{0}'"), FText::FromString(SkeletalMesh->GetName()));
		return INDEX_NONE;
	}
	if (!ensure(RenderData->LODRenderData.IsValidIndex(LODIndex)))
	{
		OutErrorMessage = FText::Format(NSLOCTEXT("GenerateSkinnedTriangleMesh", "MissingLODRenderDataFailure", "Failed to generate the skinned triangle mesh: missing LOD Render Data on Skeletal Mesh '{0}' and LOD {1}"), FText::FromString(SkeletalMesh->GetName()), LODIndex);
		return INDEX_NONE;
	}

	const FSkeletalMeshLODRenderData& LodRenderData = RenderData->LODRenderData[LODIndex];
	const int32 NumOrigVertices = LodRenderData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
	if (NumOrigVertices == 0)
	{
		// This implies the actual mesh has no positions, not that they're just missing on the CPU.
		// Rather than generate an empty object, generate an error
		OutErrorMessage = FText::Format(NSLOCTEXT("GenerateSkinnedTriangleMesh", "EmptyLODRenderDataFailure", "Failed to generate the skinned triangle mesh: empty LOD Render Data on Skeletal Mesh '{0}' at LOD {1}"), FText::FromString(SkeletalMesh->GetName()), LODIndex);
		return INDEX_NONE;
	}
	if (!ensure(NumOrigVertices == LodRenderData.SkinWeightVertexBuffer.GetNumVertices()))
	{
		OutErrorMessage = FText::Format(NSLOCTEXT("GenerateSkinnedTriangleMesh", "InvalidLODRenderData", "Failed to generate the skinned triangle mesh: invalid LOD Render Data on Skeletal Mesh '{0}' at LOD {1}"), FText::FromString(SkeletalMesh->GetName()), LODIndex);
		return INDEX_NONE;
	}
	if (!ensure(LodRenderData.StaticVertexBuffers.PositionVertexBuffer.GetVertexData() && LodRenderData.SkinWeightVertexBuffer.GetDataVertexBuffer() && LodRenderData.SkinWeightVertexBuffer.GetDataVertexBuffer()->GetWeightData()))
	{
		OutErrorMessage = FText::Format(NSLOCTEXT("GenerateSkinnedTriangleMesh", "MissingCPURenderDataFailure", "Failed to generate the skinned triangle mesh: missing CPU LOD Render Data on Skeletal Mesh '{0}' and LOD {1}"), FText::FromString(SkeletalMesh->GetName()), LODIndex);
		return INDEX_NONE;
	}	

	// Weld mesh verts to get rid of duplicates within a section. BoneData is taken from lower indexed duplicate. This matches existing cloth behavior.
	TArray<FSkinWeightInfo> SkinWeightInfos;
	LodRenderData.SkinWeightVertexBuffer.GetSkinWeights(SkinWeightInfos);
	check(SkinWeightInfos.Num() == NumOrigVertices);

	TArray<int32> OrigToWeldedVertices;
	TArray<Chaos::FVec3f> WeldedPositions;
	TArray<Chaos::FWeightedInfluenceData> BoneData;
	OrigToWeldedVertices.Reserve(NumOrigVertices);
	WeldedPositions.Reserve(NumOrigVertices);
	BoneData.Reserve(NumOrigVertices);
	for (const FSkelMeshRenderSection& Section : LodRenderData.RenderSections)
	{
		constexpr Chaos::FRealSingle ThreshSq = UE_THRESH_POINTS_ARE_SAME * UE_THRESH_POINTS_ARE_SAME;
		const int32 WeldedSectionStartIndex = WeldedPositions.Num();

		for (uint32 OrigVertexIndex = Section.BaseVertexIndex; OrigVertexIndex < Section.BaseVertexIndex + Section.NumVertices; ++OrigVertexIndex)
		{
			const Chaos::FVec3f Position(LodRenderData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(OrigVertexIndex));
			bool bIsUnique = true;
			int32 WeldedIndex = INDEX_NONE;
			for (int32 WeldedVertexIndex = WeldedSectionStartIndex; WeldedVertexIndex < WeldedPositions.Num(); ++WeldedVertexIndex)
			{
				if (Chaos::FVec3f::DistSquared(Position, WeldedPositions[WeldedVertexIndex]) <= ThreshSq)
				{
					bIsUnique = false;
					WeldedIndex = WeldedVertexIndex;
					break;
				}
			}
			if (bIsUnique)
			{
				WeldedIndex = WeldedPositions.Emplace(Position);

				// Copy bone data
				const FSkinWeightInfo& SkinWeightInfo = SkinWeightInfos[OrigVertexIndex];
				static_assert(MAX_TOTAL_INFLUENCES <= Chaos::FWeightedInfluenceData::MaxTotalInfluences);
				Chaos::FWeightedInfluenceData& NewBoneData = BoneData.AddDefaulted_GetRef();
				check(NewBoneData.NumInfluences == 0);
				for (int32 InfluenceIdx = 0; InfluenceIdx < MAX_TOTAL_INFLUENCES; ++InfluenceIdx)
				{
					if (SkinWeightInfo.InfluenceWeights[InfluenceIdx] > 0)
					{
						NewBoneData.BoneIndices[NewBoneData.NumInfluences] = Section.BoneMap[SkinWeightInfo.InfluenceBones[InfluenceIdx]];
						NewBoneData.BoneWeights[NewBoneData.NumInfluences] = (float)SkinWeightInfo.InfluenceWeights[InfluenceIdx] * UE::AnimationCore::InvMaxRawBoneWeightFloat;
						++NewBoneData.NumInfluences;
					}
				}
			}
			OrigToWeldedVertices.Emplace(WeldedIndex);
		}
	}
	check(OrigToWeldedVertices.Num() == NumOrigVertices);

	// Get UsedBones names and convert BoneIndices from RefSkeleton global indexing to local indexing.
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	
	// This is copied from WeightedLevelSetImplicitObjectBuilder... Todo refactor all this to be shared.
	TSet<int32> UsedBoneSet;
	for (int32 Idx = 0; Idx < BoneData.Num(); ++Idx)
	{
		const Chaos::FWeightedInfluenceData& Data = BoneData[Idx];
		for (uint8 InfIdx = 0; InfIdx < Data.NumInfluences; ++InfIdx)
		{
			UsedBoneSet.Add(Data.BoneIndices[InfIdx]);
		}
	}
	const TArray<int32> UsedBoneIndices = UsedBoneSet.Array();
	// Calculate the root bone index
	const int32 RootBoneIndex = CalculateCommonRootBoneIndex(RefSkeleton, UsedBoneIndices);
	const FTransform RootRefBaseInv(FMatrix(SkeletalMesh->GetRefBasesInvMatrix()[RootBoneIndex]));
	FTransform ReferenceRootTransform = RootRefBaseInv.Inverse();
	TArray<FName> UsedBones;
	TArray<FTransform> ReferenceRelativeTransforms;
	UsedBones.SetNumUninitialized(UsedBoneIndices.Num());
	ReferenceRelativeTransforms.SetNumUninitialized(UsedBoneIndices.Num());

	TMap<int32, int32> UsedBoneReverseLookup;
	for (int32 ArrayIdx = 0; ArrayIdx < UsedBoneIndices.Num(); ++ArrayIdx)
	{
		UsedBones[ArrayIdx] = RefSkeleton.GetBoneName(UsedBoneIndices[ArrayIdx]);
		ReferenceRelativeTransforms[ArrayIdx] = ReferenceRootTransform * FTransform(FMatrix(SkeletalMesh->GetRefBasesInvMatrix()[UsedBoneIndices[ArrayIdx]]));
		UsedBoneReverseLookup.Add(UsedBoneIndices[ArrayIdx], ArrayIdx);
	}

	for (int32 Idx = 0; Idx < BoneData.Num(); ++Idx)
	{
		Chaos::FWeightedInfluenceData& Data = BoneData[Idx];
		for (uint8 InfIdx = 0; InfIdx < Data.NumInfluences; ++InfIdx)
		{
			Data.BoneIndices[InfIdx] = (uint16)UsedBoneReverseLookup[Data.BoneIndices[InfIdx]];
		}

		WeldedPositions[Idx] = Chaos::FVec3f(RootRefBaseInv.TransformPosition((FVector)WeldedPositions[Idx]));
	}

	// Copy over triangles with updated indices from welding.
	TArray<uint32> OrigIndices;
	LodRenderData.MultiSizeIndexContainer.GetIndexBuffer(OrigIndices);
	check(OrigIndices.Num() % 3 == 0);
	const int32 NumTriangles = OrigIndices.Num() / 3;
	TArray<Chaos::TVec3<int32>> Elements;
	Elements.Reserve(NumTriangles);
	for (int32 TriIndex = 0; TriIndex < NumTriangles; ++TriIndex)
	{
		Elements.Emplace(
			OrigToWeldedVertices[OrigIndices[TriIndex * 3]],
			OrigToWeldedVertices[OrigIndices[TriIndex * 3 + 1]],
			OrigToWeldedVertices[OrigIndices[TriIndex * 3 + 2]]);
	}

	// Build TriangleMesh.
	Chaos::FTriangleMesh TriangleMesh(MoveTemp(Elements), 0, WeldedPositions.Num());

	// Make the SkinnedTriangleMesh
	TRefCountPtr<Chaos::FSkinnedTriangleMesh> SkinnedTriangleMesh(new Chaos::FSkinnedTriangleMesh(
		MoveTemp(TriangleMesh),
		MoveTemp(WeldedPositions),
		MoveTemp(BoneData),
		MoveTemp(UsedBones),
		MoveTemp(ReferenceRootTransform),
		MoveTemp(ReferenceRelativeTransforms)));

	// Make the SkinnedTriangleMeshElem
	FKSkinnedTriangleMeshElem SkinnedTriangleMeshElem;
	SkinnedTriangleMeshElem.SetSkinnedTriangleMesh(MoveTemp(SkinnedTriangleMesh));


	// Get or create a new body for RootBoneIndex.
	const FName RootBoneName = RefSkeleton.GetBoneName(RootBoneIndex);
	const int32 NewBodyIndex = CreateNewBody(PhysicsAsset, RootBoneName, Params);
	UBodySetup* NewBodySetup = PhysicsAsset->SkeletalBodySetups[NewBodyIndex];
	check(NewBodySetup->BoneName == RootBoneName);
	NewBodySetup->AggGeom.SkinnedTriangleMeshElems.Emplace(MoveTemp(SkinnedTriangleMeshElem));

	return NewBodyIndex;
}

void WeldBodies(UPhysicsAsset* PhysAsset, int32 BaseBodyIndex, int32 AddBodyIndex, USkeletalMeshComponent* SkelComp)
{
	if(BaseBodyIndex == INDEX_NONE || AddBodyIndex == INDEX_NONE)
		return;

	if (SkelComp == NULL || SkelComp->GetSkeletalMeshAsset() == NULL)
	{
		return;
	}

	UBodySetup* Body1 = PhysAsset->SkeletalBodySetups[BaseBodyIndex];
	int32 Bone1Index = SkelComp->GetSkeletalMeshAsset()->GetRefSkeleton().FindBoneIndex(Body1->BoneName);
	check(Bone1Index != INDEX_NONE);
	FTransform Bone1TM = SkelComp->GetBoneTransform(Bone1Index);
	Bone1TM.RemoveScaling();

	UBodySetup* Body2 = PhysAsset->SkeletalBodySetups[AddBodyIndex];
	int32 Bone2Index = SkelComp->GetSkeletalMeshAsset()->GetRefSkeleton().FindBoneIndex(Body2->BoneName);
	check(Bone2Index != INDEX_NONE);
	FTransform Bone2TM = SkelComp->GetBoneTransform(Bone2Index);
	Bone2TM.RemoveScaling();

	FTransform Bone2ToBone1TM = Bone2TM.GetRelativeTransform(Bone1TM);

	// First copy all collision info over.
	for(int32 i=0; i<Body2->AggGeom.SphereElems.Num(); i++)
	{
		int32 NewPrimIndex = Body1->AggGeom.SphereElems.Add( Body2->AggGeom.SphereElems[i] );
		Body1->AggGeom.SphereElems[NewPrimIndex].Center = Bone2ToBone1TM.TransformPosition( Body2->AggGeom.SphereElems[i].Center ); // Make transform relative to body 1 instead of body 2
	}

	for(int32 i=0; i<Body2->AggGeom.BoxElems.Num(); i++)
	{
		int32 NewPrimIndex = Body1->AggGeom.BoxElems.Add( Body2->AggGeom.BoxElems[i] );
		Body1->AggGeom.BoxElems[NewPrimIndex].SetTransform( Body2->AggGeom.BoxElems[i].GetTransform() * Bone2ToBone1TM );
	}

	for(int32 i=0; i<Body2->AggGeom.SphylElems.Num(); i++)
	{
		int32 NewPrimIndex = Body1->AggGeom.SphylElems.Add( Body2->AggGeom.SphylElems[i] );
		Body1->AggGeom.SphylElems[NewPrimIndex].SetTransform( Body2->AggGeom.SphylElems[i].GetTransform() * Bone2ToBone1TM );
	}

	for(int32 i=0; i<Body2->AggGeom.ConvexElems.Num(); i++)
	{
		FKConvexElem& Elem2 = Body2->AggGeom.ConvexElems[i];
		FTransform Elem2TM = Elem2.GetTransform() * Bone2TM;
		FTransform Elem2ToBone1TM = Elem2TM.GetRelativeTransform(Bone1TM);

		// No transform on new element - we transform all the vertices into the new ref frame instead.
		int32 NewPrimIndex = Body1->AggGeom.ConvexElems.Add( Body2->AggGeom.ConvexElems[i] );
		FKConvexElem* cElem= &Body1->AggGeom.ConvexElems[NewPrimIndex];

		for(int32 j=0; j<cElem->VertexData.Num(); j++)
		{
			cElem->VertexData[j] = Elem2ToBone1TM.TransformPosition( cElem->VertexData[j] );
		}

		// Update face data.
		cElem->UpdateElemBox();
	}

	// After changing collision, need to recreate meshes
	Body1->InvalidatePhysicsData();
	Body1->CreatePhysicsMeshes();

	// We need to update the collision disable table to shift any pairs that included body2 to include body1 instead.
	// We remove any pairs that include body2 & body1.

	for(int32 i=0; i<PhysAsset->SkeletalBodySetups.Num(); i++)
	{
		if(i == AddBodyIndex) 
			continue;

		FRigidBodyIndexPair Key(i, AddBodyIndex);

		if( PhysAsset->CollisionDisableTable.Find(Key) )
		{
			PhysAsset->CollisionDisableTable.Remove(Key);

			// Only re-add pair if its not between 'base' and 'add' bodies.
			if(i != BaseBodyIndex)
			{
				FRigidBodyIndexPair NewKey(i, BaseBodyIndex);
				PhysAsset->CollisionDisableTable.Add(NewKey, 0);
			}
		}
	}

	// Make a sensible guess for the other flags
	ECollisionEnabled::Type NewCollisionEnabled = FMath::Min(Body1->DefaultInstance.GetCollisionEnabled(), Body2->DefaultInstance.GetCollisionEnabled());
	Body1->DefaultInstance.SetCollisionEnabled(NewCollisionEnabled);

	// if different
	if (Body1->PhysicsType != Body2->PhysicsType)
	{
		// i don't think this is necessarily good, but I think better than default
		Body1->PhysicsType = FMath::Max(Body1->PhysicsType, Body2->PhysicsType);
	}

	// Then deal with any constraints.

	TArray<int32>	Body2Constraints;
	PhysAsset->BodyFindConstraints(AddBodyIndex, Body2Constraints);

	while( Body2Constraints.Num() > 0 )
	{
		int32 ConstraintIndex = Body2Constraints[0];
		FConstraintInstance& Instance = PhysAsset->ConstraintSetup[ConstraintIndex]->DefaultInstance;

		FName OtherBodyName;
		if( Instance.ConstraintBone1 == Body2->BoneName )
			OtherBodyName = Instance.ConstraintBone2;
		else
			OtherBodyName = Instance.ConstraintBone1;

		// If this is a constraint between the two bodies we are welding, we just destroy it.
		if(OtherBodyName == Body1->BoneName)
		{
			DestroyConstraint(PhysAsset, ConstraintIndex);
		}
		else // Otherwise, we reconnect it to body1 (the 'base' body) instead of body2 (the 'weldee').
		{
			if(Instance.ConstraintBone2 == Body2->BoneName)
			{
				Instance.ConstraintBone2 = Body1->BoneName;

				FTransform ConFrame = Instance.GetRefFrame(EConstraintFrame::Frame2);
				Instance.SetRefFrame(EConstraintFrame::Frame2, ConFrame * FTransform(Bone2ToBone1TM));
			}
			else
			{
				Instance.ConstraintBone1 = Body1->BoneName;

				FTransform ConFrame = Instance.GetRefFrame(EConstraintFrame::Frame1);
				Instance.SetRefFrame(EConstraintFrame::Frame1, ConFrame * FTransform(Bone2ToBone1TM));
			}
		}

		// See if we have any more constraints to body2.
		PhysAsset->BodyFindConstraints(AddBodyIndex, Body2Constraints);
	}

	// Finally remove the body
	DestroyBody(PhysAsset, AddBodyIndex);
}

int32 CreateNewConstraint(UPhysicsAsset* PhysAsset, FName InConstraintName, UPhysicsConstraintTemplate* InConstraintSetup)
{
	// constraintClass must be a subclass of UPhysicsConstraintTemplate
	int32 ConstraintIndex = PhysAsset->FindConstraintIndex(InConstraintName);
	if(ConstraintIndex != INDEX_NONE)
	{
		return ConstraintIndex;
	}

	if (!ensure(CanCreateConstraints()))
	{
		return INDEX_NONE;
	}

	UPhysicsConstraintTemplate* NewConstraintSetup = NewObject<UPhysicsConstraintTemplate>(PhysAsset, NAME_None, RF_Transactional);
	if(InConstraintSetup)
	{
		NewConstraintSetup->DefaultInstance.CopyConstraintParamsFrom( &InConstraintSetup->DefaultInstance );
	}

	int32 ConstraintSetupIndex = PhysAsset->ConstraintSetup.Add( NewConstraintSetup );
	NewConstraintSetup->DefaultInstance.JointName = InConstraintName;

	return ConstraintSetupIndex;
}

void DestroyConstraint(UPhysicsAsset* PhysAsset, int32 ConstraintIndex)
{
	check(PhysAsset);
	PhysAsset->ConstraintSetup.RemoveAt(ConstraintIndex);
}

int32 CreateNewBody(UPhysicsAsset* const PhysAsset, const FName InBodyName, const FPhysAssetCreateParams& Params)
{
	check(PhysAsset);

	int32 BodyIndex = PhysAsset->FindBodyIndex(InBodyName);
	if(BodyIndex != INDEX_NONE)
	{
		return BodyIndex; // if we already have one for this name - just return that.
	}

	BodyIndex = PhysAsset->SkeletalBodySetups.AddDefaulted();
	CreateNewBodyAtIndexInternal(PhysAsset, InBodyName, Params, BodyIndex);

	return BodyIndex; // Return index of new body.
}

void RecreateBody(UPhysicsAsset* const PhysAsset, const FName InBodyName, const FPhysAssetCreateParams& Params, const int32 BodyIndex)
{
	RemoveBodyFromCollisionTableInternal(PhysAsset, BodyIndex, false);
	PhysAsset->SkeletalBodySetups[BodyIndex] = TObjectPtr<USkeletalBodySetup>(); // Reset TObjectPtr to body setup that will be replaced.
	CreateNewBodyAtIndexInternal(PhysAsset, InBodyName, Params, BodyIndex);
}

void DestroyBody(UPhysicsAsset* const PhysAsset, const int32 bodyIndex)
{
	RemoveBodyFromCollisionTableInternal(PhysAsset, bodyIndex, true);
	RemoveBodyConstraintsInternal(PhysAsset, bodyIndex);
	
	// Remove pointer from array. Actual objects will be garbage collected.
	PhysAsset->SkeletalBodySetups.RemoveAt(bodyIndex);

	PhysAsset->UpdateBodySetupIndexMap();
	// Update body indices.
	PhysAsset->UpdateBoundsBodiesArray();
}

bool CanCreateConstraints()
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(EAssetClassAction::CreateAsset);
	if (UPhysicsConstraintTemplate::StaticClass() && AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
	{
		return AssetClassPermissionList->PassesFilter(UPhysicsConstraintTemplate::StaticClass()->GetPathName());
	}
	return true;
}

void SanitizeRestrictedContent(UPhysicsAsset* PhysAsset)
{
	check(PhysAsset);
}

// Explicitly instantiate and export covariance matrix calc
template PHYSICSUTILITIES_API FMatrix ComputeViewCovarianceMatrix(TArrayView<const UE::Math::TVector<float>, int> PointView);
template PHYSICSUTILITIES_API FMatrix ComputeViewCovarianceMatrix(TArrayView<const UE::Math::TVector<double>, int> PointView);

}; // namespace FPhysicsAssetUtils

IMPLEMENT_MODULE(FDefaultModuleImpl, PhysicsUtilities)

