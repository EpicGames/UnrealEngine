// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinnedBoneTriangleCache.h"
#include "Components/SkeletalMeshComponent.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"

FSkinnedBoneTriangleCache::FSkinnedBoneTriangleCache(const USkeletalMesh& InSkeletalMesh, const FPhysAssetCreateParams& Params)
	: SkeletalMesh(InSkeletalMesh),
	StaticLODModel(*SkeletalMesh.GetImportedModel()),
	RenderData(*SkeletalMesh.GetResourceForRendering()),
	VertexBuffer(RenderData.LODRenderData[0].StaticVertexBuffers.PositionVertexBuffer),
	InfluenceHeuristic(Params.VertWeight),
	BoneIndexToInfluencedVertices(),
	BoneIndexToTriangles(),
	LODModelIndexBufferInOrder()
{
	
}

void FSkinnedBoneTriangleCache::BuildCache()
{
	BuildInfluencedIndexSetForEachBone();
	BuildOwnedTrianglesSetForEachBone();
}

void FSkinnedBoneTriangleCache::GetVerticesAndIndicesForBoneInternal(const int32 TrianglesBoneIndex, const int32 TransformBoneIndex, TArray<FVector3f>& OutVertexPositions, TArray<uint32>& OutIndices, TArray<uint32>& OutLocalToSkinnedVertexIndex) const
{
	const FTriangleArray* TriangleArrayPointer = BoneIndexToTriangles.Find(TrianglesBoneIndex);
	if ( !TriangleArrayPointer )
	{
		return;
	}

	const FTriangleArray& TrianglesForBone = *TriangleArrayPointer;
	const FMatrix ComponentToBoneMatrix = BoneTransformMatrix(TransformBoneIndex);

	TMap<FSkinnedVertexIndex, uint32> SkinnedVertIndexToOutputIndex;

	for ( int32 CurrentIndex = 0; CurrentIndex < TrianglesForBone.Num(); ++CurrentIndex )
	{
		const FTriangleIndex TriangleIndex = TrianglesForBone[CurrentIndex];
		check(BufferIndexForTri(TriangleIndex, 2) < static_cast<uint32>(LODModelIndexBufferInOrder.Num()));

		for ( int32 TriangleVert = 0; TriangleVert < 3; ++TriangleVert )
		{
			const uint32 BufferIndex = BufferIndexForTri(TriangleIndex, TriangleVert);
			const FSkinnedVertexIndex VertIndex = LODModelIndexBufferInOrder[BufferIndex];

			// If we haven't seen this vertex before, we need to add it to our output positions.
			if ( !SkinnedVertIndexToOutputIndex.Contains(VertIndex) )
			{
				OutVertexPositions.Add(VertexPosition(VertIndex, ComponentToBoneMatrix));
				SkinnedVertIndexToOutputIndex.Add(VertIndex, static_cast<uint32>(OutVertexPositions.Num() - 1));
				OutLocalToSkinnedVertexIndex.Add(VertIndex);
			}

			// Map the skinned vert index to the index in our output array.
			OutIndices.Add(SkinnedVertIndexToOutputIndex[VertIndex]);
		}
	}
}

void FSkinnedBoneTriangleCache::GetVerticesAndIndicesForBone(const int32 BoneIndex, TArray<FVector3f>& OutVertexPositions, TArray<uint32>& OutIndices) const
{
	OutVertexPositions.Reset();
	OutIndices.Reset();
	TArray<uint32> UnusedLocalToSkinnedVertexIndex;
	return GetVerticesAndIndicesForBoneInternal(BoneIndex, BoneIndex, OutVertexPositions, OutIndices, UnusedLocalToSkinnedVertexIndex);
}

void FSkinnedBoneTriangleCache::GetVerticesAndIndicesForBones(const int32 PrimaryBoneIndex, const TArray<int32>& BoneIndices, TArray<FVector3f>& OutVertexPositions, TArray<uint32>& OutIndices, TArray<uint32>& OutLocalToSkinnedVertexIndex) const
{
	OutVertexPositions.Reset();
	OutIndices.Reset();
	OutLocalToSkinnedVertexIndex.Reset();
	for (const int32 BoneIndex : BoneIndices)
	{
		GetVerticesAndIndicesForBoneInternal(BoneIndex, PrimaryBoneIndex, OutVertexPositions, OutIndices, OutLocalToSkinnedVertexIndex);
	}
}

void FSkinnedBoneTriangleCache::BuildInfluencedIndexSetForEachBone()
{
	const uint32 TotalVertices = VertexBuffer.GetNumVertices();

	for ( FSkinnedVertexIndex VertIndex = 0; VertIndex < TotalVertices; ++VertIndex )
	{
		AddIndexToInfluencerBoneSets(VertIndex);
	}
}

void FSkinnedBoneTriangleCache::AddIndexToInfluencerBoneSets(const FSkinnedVertexIndex VertIndex)
{
	int32 SectionIndex;
	int32 SoftVertIndex;
	RenderData.LODRenderData[0].GetSectionFromVertexIndex(VertIndex, SectionIndex, SoftVertIndex);

	const FSkelMeshSection& Section = StaticLODModel.LODModels[0].Sections[SectionIndex];
	const FSoftSkinVertex& SoftVert = Section.SoftVertices[SoftVertIndex];
	const uint16 MaxWeight = InfluenceHeuristic == EVW_DominantWeight ? SoftVert.GetMaximumWeight() : 0;

	for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
	{
		const uint16 InfluenceWeight = SoftVert.InfluenceWeights[InfluenceIndex];

		if ( InfluenceHeuristic == EVW_DominantWeight )
		{
			if ( InfluenceWeight < MaxWeight )
			{
				continue;
			}
		}
		else
		{
			if ( InfluenceWeight < 1 )
			{
				continue;
			}
		}

		const FBoneIndexType BoneMapIndex = SoftVert.InfluenceBones[InfluenceIndex];
		const int32 ActualBoneIndex = Section.BoneMap[BoneMapIndex];

		FInfluencedVerticesSet& InfluencedVertexIndicesForBone = BoneIndexToInfluencedVertices.FindOrAdd(ActualBoneIndex);
		InfluencedVertexIndicesForBone.Add(VertIndex);
	}
}

void FSkinnedBoneTriangleCache::BuildOwnedTrianglesSetForEachBone()
{
	LODModelIndexBufferInOrder.Empty();
	RenderData.LODRenderData[0].MultiSizeIndexContainer.GetIndexBuffer(LODModelIndexBufferInOrder);

	// We assume that each triplet of indices in the index buffer forms a triangle.
	check(LODModelIndexBufferInOrder.Num() % 3 == 0);

	const uint32 TotalTriangles = LODModelIndexBufferInOrder.Num() / 3;

	for ( FTriangleIndex TriangleIndex = 0; TriangleIndex < TotalTriangles; ++TriangleIndex )
	{
		AddTriangleIndexToOwnerBoneSets(TriangleIndex);
	}
}

void FSkinnedBoneTriangleCache::AddTriangleIndexToOwnerBoneSets(const FTriangleIndex TriangleIndex)
{
	check(BufferIndexForTri(TriangleIndex, 2) < static_cast<uint32>(LODModelIndexBufferInOrder.Num()));

	const FSkinnedVertexIndex TriangleVertices[3] =
	{
		LODModelIndexBufferInOrder[BufferIndexForTri(TriangleIndex, 0)],
		LODModelIndexBufferInOrder[BufferIndexForTri(TriangleIndex, 1)],
		LODModelIndexBufferInOrder[BufferIndexForTri(TriangleIndex, 2)]
	};

	for ( FBoneIndexToInfluencedVertices::TConstIterator ConstIt = BoneIndexToInfluencedVertices.CreateConstIterator(); ConstIt; ++ConstIt )
	{
		const FBoneIndex CurrentBoneIndex = ConstIt.Key();
		const FInfluencedVerticesSet& InfluencedVerticesForBone = ConstIt.Value();
		
		for (int32 TriangleVert = 0; TriangleVert < 3; ++TriangleVert)
		{
			if (InfluencedVerticesForBone.Contains(TriangleVertices[TriangleVert]))
			{
				FTriangleArray& TriangleArrayForBone = BoneIndexToTriangles.FindOrAdd(CurrentBoneIndex);
				TriangleArrayForBone.Add(TriangleIndex);

				break;
			}
		}
	}
}

FVector3f FSkinnedBoneTriangleCache::VertexPosition(const FSkinnedVertexIndex VertIndex, const FMatrix& ComponentToBoneMatrix) const
{
	const FVector3f& Position = VertexBuffer.VertexPosition(VertIndex);
	return FVector4f(ComponentToBoneMatrix.TransformPosition((FVector)Position));
}

FMatrix FSkinnedBoneTriangleCache::BoneTransformMatrix(FBoneIndex BoneIndex) const
{
	return FMatrix(SkeletalMesh.GetRefBasesInvMatrix()[BoneIndex]);
}
