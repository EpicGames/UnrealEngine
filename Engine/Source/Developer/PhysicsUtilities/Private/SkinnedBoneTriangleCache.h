// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "SkeletalRenderPublic.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "PhysicsAssetUtils.h"

/**
 * To build a convex hull for a bone, DecomposeMeshToHulls() requires an array of vertex positions
 * and an array of indices to these vertices (used to specify the triangles that make up the
 * surface of that bone).
 * 
 * In order to provide this information, we need to know which triangles a bone "owns". A bone
 * owns a triangle if at least one vertex used in the triangle is influenced by the bone. Once we
 * know which triangles the bone owns, we can provide the arrays of positions and indices.
 * 
 * To find out which triangles a bone owns, we first need to build a set of which vertices are
 * influenced by that bone. Once we have this, we can check each triangle in the mesh to see if
 * the bone owns it; if the bone does own the triangle, the index of that triangle can then be
 * stored in another array.
 * 
 * Finally, knowing all the triangles that a bone owns, we can generate an array of all the
 * vertex positions used by these triangles, along with a corresponding local index array.
 */
class FSkinnedBoneTriangleCache
{
public:
	explicit FSkinnedBoneTriangleCache(const USkeletalMesh& InSkeletalMesh, const FPhysAssetCreateParams& Params);

	void BuildCache();
	void GetVerticesAndIndicesForBone(const int32 BoneIndex, TArray<FVector3f>& OutVertexPositions, TArray<uint32>& OutIndices) const;
	// Vertex positions will be transformed by PrimaryBoneIndex's transform matrix.
	void GetVerticesAndIndicesForBones(const int32 PrimaryBoneIndex, const TArray<int32>& BoneIndices, TArray<FVector3f>& OutVertexPositions, TArray<uint32>& OutIndices, TArray<uint32>& OutLocalToSkinnedVertexIndex) const;

	bool IsBuilt() const { return !BoneIndexToInfluencedVertices.IsEmpty() || !BoneIndexToTriangles.IsEmpty(); }

private:
	typedef int32 FBoneIndex;
	typedef uint32 FSkinnedVertexIndex;
	typedef uint32 FTriangleIndex;

	typedef TSet<FSkinnedVertexIndex> FInfluencedVerticesSet;
	typedef TMap<FBoneIndex, FInfluencedVerticesSet> FBoneIndexToInfluencedVertices;

	typedef TArray<FTriangleIndex> FTriangleArray;
	typedef TMap<FBoneIndex, FTriangleArray> FBoneIndexToTriangles;

	inline static uint32 BufferIndexForTri(const FTriangleIndex TriangleIndex, const int32 TriangleVertex = 0)
	{
		return (3 * TriangleIndex) + TriangleVertex;
	}

	FMatrix BoneTransformMatrix(FBoneIndex BoneIndex) const;

	/**
	* In order to generate a set of triangles owned by each bone, we need to build up a set
	* of vertices influenced by each bone. This boils down to creating set of vertex indices,
	* which index into the LOD model's vertex array. If a bone's set contains an index, the
	* corresponding vertex is influenced by that bone.
	*/
	void BuildInfluencedIndexSetForEachBone();

	/** Here we pass in an index to the vertex array, and determine which sets it should be added to. */
	void AddIndexToInfluencerBoneSets(const FSkinnedVertexIndex VertIndex);

	/** Once the sets of influences vertices have been computed, the sets of owned triangles can be built. */
	void BuildOwnedTrianglesSetForEachBone();

	/** Here, each triangle index is added to its appropriate sets. */
	void AddTriangleIndexToOwnerBoneSets(const FTriangleIndex TriangleIndex);

	FVector3f VertexPosition(const FSkinnedVertexIndex VertIndex, const FMatrix& ComponentToBoneMatrix) const;

	void GetVerticesAndIndicesForBoneInternal(const int32 TrianglesBoneIndex, const int32 TransformBoneIndex, TArray<FVector3f>& OutVertexPositions, TArray<uint32>& OutIndices, TArray<uint32>& OutLocalToSkinnedVertexIndex) const;

	// Inputs
	const USkeletalMesh& SkeletalMesh;

	// Computed from inputs
	const class FSkeletalMeshModel& StaticLODModel;
	const class FSkeletalMeshRenderData& RenderData;
	const class FPositionVertexBuffer& VertexBuffer;

	/**
	* EVW_AnyWeight:		Any vertex influenced by the bone is included in the set.
	* EVW_DominantWeight:	Only vertices where the given bone has the highest influence are included in the set.
	*						Note that if two bones tie for the largest influence on a vertex, the vertex is included in both sets.
	*/
	EPhysAssetFitVertWeight InfluenceHeuristic;

	// Internal
	FBoneIndexToInfluencedVertices BoneIndexToInfluencedVertices;
	FBoneIndexToTriangles BoneIndexToTriangles;
	TArray<FSkinnedVertexIndex> LODModelIndexBufferInOrder;
};
