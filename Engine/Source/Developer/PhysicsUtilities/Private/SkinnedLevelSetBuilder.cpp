// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinnedLevelSetBuilder.h"
#include "SkinnedBoneTriangleCache.h"
#include "LevelSetHelpers.h"

#include "BoneWeights.h"
#include "Chaos/ArrayND.h"
#include "Chaos/Levelset.h"
#include "Chaos/Plane.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Math/Poisson.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/SkeletalMesh.h"
#include "Implicit/SweepingMeshSDF.h"
#include "Implicit/Solidify.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/SkinnedLevelSetElem.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"

FSkinnedLevelSetBuilder::FSkinnedLevelSetBuilder(const USkeletalMesh& InSkeletalMesh, const FSkinnedBoneTriangleCache& InTriangleCache, const int32 InRootBoneIndex)
	:SkeletalMesh(InSkeletalMesh)
	, TriangleCache(InTriangleCache)
	, RootBoneIndex(InRootBoneIndex)
	, StaticLODModel(SkeletalMesh.GetImportedModel())
	, RenderData(SkeletalMesh.GetResourceForRendering())
{
	check(StaticLODModel);
	check(RenderData);
}

bool FSkinnedLevelSetBuilder::InitializeSkinnedLevelset(const FPhysAssetCreateParams& Params, const TArray<int32>& BoneIndices, TArray<uint32>& OutOrigIndices)
{
	check(SkeletalMesh.GetRefSkeleton().IsValidRawIndex(RootBoneIndex));

	TArray<FVector3f> Positions;
	TArray<uint32> Indices;
	if (Params.VertWeight == EVW_AnyWeight)
	{
		// Want to use EVW_DominantWeight for initial levelset
		// Build a different cache
		FPhysAssetCreateParams ParamsCopy;
		ParamsCopy.VertWeight = EVW_DominantWeight;
		FSkinnedBoneTriangleCache DominantCache(SkeletalMesh, ParamsCopy);
		DominantCache.BuildCache();
		DominantCache.GetVerticesAndIndicesForBones(RootBoneIndex, BoneIndices, Positions, Indices, OutOrigIndices);
	}
	else
	{

		TriangleCache.GetVerticesAndIndicesForBones(RootBoneIndex, BoneIndices, Positions, Indices, OutOrigIndices);
	}

	if (Positions.Num())
	{
		UE::Geometry::FDynamicMesh3 DynamicMesh;
		LevelSetHelpers::CreateDynamicMesh(Positions, Indices, DynamicMesh);

		const bool bOK = LevelSetHelpers::CreateLevelSetForMesh(DynamicMesh, Params.LevelSetResolution, LevelSet);
		if (bOK)
		{
			GenerateGrid(Params.LatticeResolution, LevelSet->BoundingBox());
		}
		else
		{
			FMessageLog EditorErrors("EditorErrors");
			EditorErrors.Warning(NSLOCTEXT("PhysicsAssetUtils", "SkinnedLevelSetError", "An error occurred creating root skinned level set."));
			EditorErrors.Open();
		}
		return bOK;
	}
	FMessageLog EditorErrors("EditorErrors");
	EditorErrors.Warning(NSLOCTEXT("PhysicsAssetUtils", "LevelSetNoPositions", "Unable to create a level set for the given bone as there are no vertices associated with the bone."));
	EditorErrors.Open();
	return false;
}

static float CalculateTriangleWeight(const UE::Geometry::FTriangle3d& Triangle, const FVector3f& TriangleWeights, const Chaos::TVector<double, 3>& Location)
{
	if (TriangleWeights.X == 0.f)
	{
		if (TriangleWeights.Y == 0.f)
		{
			check(TriangleWeights.Z != 0.f);
			return TriangleWeights.Z;
		}
		else if (TriangleWeights.Z == 0.f)
		{
			check(TriangleWeights.Y != 0.f);
			return TriangleWeights.Y;
		}
		else
		{
			double Alpha;
			Chaos::FindClosestPointAndAlphaOnLineSegment(Chaos::FVec3(Triangle.V[1]), Chaos::FVec3(Triangle.V[2]), Location, Alpha);
			return (1. - Alpha) * TriangleWeights.Y + Alpha * TriangleWeights.Z;
		}
	}
	else if (TriangleWeights.Y == 0.f)
	{
		if (TriangleWeights.Z == 0.f)
		{
			check(TriangleWeights.X != 0.f);
			return TriangleWeights.X;
		}
		else
		{
			double Alpha;
			Chaos::FindClosestPointAndAlphaOnLineSegment(Chaos::FVec3(Triangle.V[0]), Chaos::FVec3(Triangle.V[2]), Location, Alpha);
			return (1. - Alpha) * TriangleWeights.X + Alpha * TriangleWeights.Z;
		}
	}
	else if (TriangleWeights.Z == 0.f)
	{
		double Alpha;
		Chaos::FindClosestPointAndAlphaOnLineSegment(Chaos::FVec3(Triangle.V[0]), Chaos::FVec3(Triangle.V[1]), Location, Alpha);
		return (1. - Alpha) * TriangleWeights.X + Alpha * TriangleWeights.Y;

	}
	else
	{
		// Get Barycentric coordinates for closest point
		UE::Geometry::TDistPoint3Triangle3<double> TriQuery(Location, Triangle);
		TriQuery.ComputeResult();
		return TriQuery.TriangleBaryCoords.X * TriangleWeights.X +
			TriQuery.TriangleBaryCoords.Y * TriangleWeights.Y + TriQuery.TriangleBaryCoords.Z * TriangleWeights.Z;
	}
}

void FSkinnedLevelSetBuilder::AddBoneInfluence(int32 PrimaryBoneIndex, const TArray<int32>& AllBonesForInfluence)
{
	// Need to generate weights for grid points corresponding with this bone.
	// Will laplacian diffuse the weights from surface to bone interior.

	// Get Triangles that correspond with these bones.
	TArray<FVector3f> AllVerts;
	TArray<uint32> AllIndices;
	TArray<uint32> AllOrigIndices;
	TriangleCache.GetVerticesAndIndicesForBones(RootBoneIndex, AllBonesForInfluence, AllVerts, AllIndices, AllOrigIndices);

	// Strip any vertices that are outside the levelset.
	TArray<FVector3f> Verts;
	Verts.Reserve(AllVerts.Num());
	TArray<uint32> Indices;
	Indices.Reserve(AllIndices.Num());
	TArray<uint32> OrigIndices;
	OrigIndices.Reserve(AllOrigIndices.Num());
	TArray<int32> AllVertToVerts;
	AllVertToVerts.SetNumUninitialized(AllVerts.Num());
	const double MaxInsidePhi = LevelSet->GetGrid().Dx().GetMax() * .5;

	for (int32 AllVertIdx = 0; AllVertIdx < AllVerts.Num(); ++AllVertIdx)
	{
		if (LevelSet->SignedDistance(Chaos::FVec3(AllVerts[AllVertIdx])) < MaxInsidePhi)
		{
			const int32 VertIdx = Verts.Add(AllVerts[AllVertIdx]);
			OrigIndices.Add(AllOrigIndices[AllVertIdx]);
			AllVertToVerts[AllVertIdx] = VertIdx;
		}
		else
		{
			AllVertToVerts[AllVertIdx] = INDEX_NONE;
		}
	}
	check(AllIndices.Num() % 3 == 0);
	const int32 NumTriangles = AllIndices.Num() / 3;
	for (int32 AllTriIdx = 0; AllTriIdx < NumTriangles; ++AllTriIdx)
	{
		const int32 VertIdx0 = AllVertToVerts[AllIndices[AllTriIdx * 3]];
		const int32 VertIdx1 = AllVertToVerts[AllIndices[AllTriIdx * 3 + 1]];
		const int32 VertIdx2 = AllVertToVerts[AllIndices[AllTriIdx * 3 + 2]];
		if (VertIdx0 != INDEX_NONE && VertIdx1 != INDEX_NONE && VertIdx2 != INDEX_NONE)
		{
			Indices.Add(VertIdx0);
			Indices.Add(VertIdx1);
			Indices.Add(VertIdx2);
		}
	}
	
	const Chaos::TUniformGrid<double, 3>& LatticeGrid = GetGrid();
	const Chaos::TUniformGrid<double, 3>& LevelsetGrid = LevelSet->GetGrid();
	// Surface is almost certainly not closed, but we want to diffuse differently inside vs outside, so we
	// need a closed surface. Using solidify to do this.
	UE::Geometry::FDynamicMesh3 DynamicMesh;
	LevelSetHelpers::CreateDynamicMesh(Verts, Indices, DynamicMesh);
	UE::Geometry::TMeshAABBTree3<UE::Geometry::FDynamicMesh3> Spatial(&DynamicMesh);
	UE::Geometry::TFastWindingTree<UE::Geometry::FDynamicMesh3> FastWinding(&Spatial);
	UE::Geometry::TImplicitSolidify<UE::Geometry::FDynamicMesh3> Solidify(&DynamicMesh, &Spatial, &FastWinding);
	constexpr int32 NumExpandCells = 1;
	const double SolidifyCellSize = LevelsetGrid.Dx().Min();
	Solidify.ExtendBounds = NumExpandCells * SolidifyCellSize;
	Solidify.MeshCellSize = SolidifyCellSize;
	Solidify.WindingThreshold = 0.5;
	Solidify.SurfaceSearchSteps = 3;
	Solidify.bSolidAtBoundaries = true;
	const UE::Geometry::FDynamicMesh3 SolidMesh(&Solidify.Generate());
	UE::Geometry::TMeshAABBTree3<UE::Geometry::FDynamicMesh3> SolidSpatial(&SolidMesh);
	UE::Geometry::TFastWindingTree<UE::Geometry::FDynamicMesh3> SolidFastWinding(&SolidSpatial);
	constexpr int32 OuterWeightExpandCells = 4; // Diffuse out this far outside mesh
	constexpr int32 SurfaceExpandCells = 1; // Consider points this far from the surface of the mesh to be on the surface (dirichlet boundary)
	UE::Geometry::IMeshSpatial::FQueryOptions QueryOptions;
	QueryOptions.MaxDistance = (double)LatticeGrid.Dx().Max() * FMath::Max(OuterWeightExpandCells, SurfaceExpandCells);// Used for initial query against solidified mesh

	const double OuterWeightExpandDistSq = FMath::Square((double)LatticeGrid.Dx().Max() * OuterWeightExpandCells);
	const double SurfaceExpandDistSq = FMath::Square((double)LatticeGrid.Dx().Max() * SurfaceExpandCells);
	TSet<int32> BonesSet(AllBonesForInfluence);
	// Create sub-grid for the poisson solve.
	UE::Geometry::FAxisAlignedBox3d SolidBBox = SolidSpatial.GetBoundingBox();
	const Chaos::TVec3<int32> MinCornerCell = (LatticeGrid.Cell(SolidBBox.Min) - Chaos::TVec3<int32>(OuterWeightExpandCells)).ComponentwiseMax(Chaos::TVec3<int32>(0));
	const Chaos::TVec3<int32> MaxCornerCell = (LatticeGrid.Cell(SolidBBox.Max) + Chaos::TVec3<int32>(OuterWeightExpandCells)).ComponentwiseMin(LatticeGrid.Counts() - Chaos::TVec3<int32>(1));
	const Chaos::TUniformGrid<double, 3> SubGrid = LatticeGrid.SubGrid(MinCornerCell, MaxCornerCell);
	// Get constrained nodes and weights for poisson solve
	TArray<int32> ConstrainedNodes;
	TArray<float> ConstrainedWeights;
	const Chaos::TVec3<int32> NodeCounts = SubGrid.NodeCounts();
	TSet<int32> OutsideWeighted;
	for (int32 I = 0; I < NodeCounts.X; ++I)
	{
		for (int32 J = 0; J < NodeCounts.Y; ++J)
		{
			for (int32 K = 0; K < NodeCounts.Z; ++K)
			{
				const Chaos::TVector<int32, 3> Index(I, J, K);
				const Chaos::TVector<double, 3> Location = SubGrid.Node(Index);
				// Determine if close to surface of solid mesh.
				double NearestDistSq = std::numeric_limits<double>::max();
				const int32 NearestSolidTriangle = SolidSpatial.FindNearestTriangle(Location, NearestDistSq, QueryOptions);
				if (NearestSolidTriangle == IndexConstants::InvalidID || NearestDistSq > SurfaceExpandCells)
				{
					// Not near surface.
					// Determine if inside or outside
					const bool IsInside = SolidFastWinding.IsInside(Location);
					const bool IsGridBoundary = I == 0 || I == NodeCounts.X - 1 || J == 0 || J == NodeCounts.Y - 1 || K == 0 || K == NodeCounts.Z - 1;
					if (!IsInside)
					{
						const bool IsOutsideOuterExpand = NearestSolidTriangle == IndexConstants::InvalidID || NearestDistSq > OuterWeightExpandDistSq;
						if (IsOutsideOuterExpand)
						{
							// Outside the distance to diffuse this weight. Mark as dirichlet with weight = 0
							ConstrainedNodes.Add(SubGrid.FlatIndex(Index, true));
							ConstrainedWeights.Add(0.f);
							continue;
						}
						else if (!IsGridBoundary)
						{
							OutsideWeighted.Add(SubGrid.FlatIndex(Index, true));
							continue;
						}
					}
					else if (!IsGridBoundary)
					{
						continue;
					}
				}
				// Need to find closest triangle on original mesh to get weights. No distance limit since we might be in a hole that was filled
				// by solidify. 
				const int32 NearestTriangle = Spatial.FindNearestTriangle(Location, NearestDistSq);
				const UE::Geometry::FIndex3i& TriangleIndices = DynamicMesh.GetTriangle(NearestTriangle);
				const FVector3f TriangleWeights(GetWeightForIndices(BonesSet, OrigIndices[TriangleIndices.A]), GetWeightForIndices(BonesSet, OrigIndices[TriangleIndices.B]), GetWeightForIndices(BonesSet, OrigIndices[TriangleIndices.C]));

				UE::Geometry::FTriangle3d Triangle;
				DynamicMesh.GetTriVertices(NearestTriangle, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
				const float Weight = CalculateTriangleWeight(Triangle, TriangleWeights, Location);				
				check(Weight != 0.f);
				ConstrainedNodes.Add(SubGrid.FlatIndex(Index, true));
				ConstrainedWeights.Add(Weight);
			}
		}
	}
	TArray<float> Weights;
	Weights.SetNumUninitialized(SubGrid.GetNumNodes());
	constexpr int32 MaxIter = 10000;
	constexpr float Res = 1e-4f;
	constexpr bool bCheckResidual = true;
	constexpr int32 MinParallelBatchSize = 10000;
	Chaos::PoissonSolve<float, double, true>(ConstrainedNodes, ConstrainedWeights, SubGrid, MaxIter, Res, Weights, bCheckResidual, MinParallelBatchSize);
	for (int32 Idx = 0; Idx < Weights.Num(); ++Idx)
	{
		if (Weights[Idx] > 0)
		{
			Chaos::TVec3<int32> SubIndex;
			SubGrid.FlatToMultiIndex(Idx, SubIndex, true);
			const int32 FullIndex = LatticeGrid.FlatIndex(SubIndex + MinCornerCell, true);
			AddInfluence(FullIndex, PrimaryBoneIndex, Weights[Idx], OutsideWeighted.Contains(Idx));
		}
	}
}

FKSkinnedLevelSetElem FSkinnedLevelSetBuilder::CreateSkinnedLevelSetElem()
{
	FinalizeInfluences([this](int32 BoneIndex)
		{
			return SkeletalMesh.GetRefSkeleton().GetBoneName(BoneIndex);
		},
		[this](int32 BoneIndex)
		{
			if (BoneIndex == INDEX_NONE)
			{
				return FTransform(FMatrix(SkeletalMesh.GetRefBasesInvMatrix()[RootBoneIndex]));
			}
		return FTransform(FMatrix(SkeletalMesh.GetRefBasesInvMatrix()[BoneIndex]));
		}
		);
	TRefCountPtr<Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet> > WeightedLevelSet = Generate(MoveTemp(LevelSet));
	FKSkinnedLevelSetElem Ret;
	Ret.SetWeightedLevelSet(MoveTemp(WeightedLevelSet));
	return Ret;
}

float FSkinnedLevelSetBuilder::GetWeightForIndices(const TSet<int32>& BoneIndices, uint32 VertIndex) const
{
	int32 SectionIndex;
	int32 SoftVertIndex;
	RenderData->LODRenderData[0].GetSectionFromVertexIndex(VertIndex, SectionIndex, SoftVertIndex);
	const FSkelMeshSection& Section = StaticLODModel->LODModels[0].Sections[SectionIndex];
	const FSoftSkinVertex& SoftVert = Section.SoftVertices[SoftVertIndex];

	float Weight = 0.f;
	for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
	{
		const FBoneIndexType BoneMapIndex = SoftVert.InfluenceBones[InfluenceIndex];
		const int32 ActualBoneIndex = Section.BoneMap[BoneMapIndex];
		if (BoneIndices.Contains(ActualBoneIndex))
		{
			// Use max weight for all bones.
			Weight = FMath::Max(Weight, (float)SoftVert.InfluenceWeights[InfluenceIndex] * UE::AnimationCore::InvMaxRawBoneWeightFloat);
		}
	}

	return Weight;
}

void FSkinnedLevelSetBuilder::GetInfluencingBones(const TArray<uint32>& SkinnedVertexIndices, TSet<int32>& Bones)
{
	for (uint32 SkinnedVertIndex : SkinnedVertexIndices)
	{
		int32 SectionIndex;
		int32 SoftVertIndex;
		RenderData->LODRenderData[0].GetSectionFromVertexIndex(SkinnedVertIndex, SectionIndex, SoftVertIndex);

		const FSkelMeshSection& Section = StaticLODModel->LODModels[0].Sections[SectionIndex];
		const FSoftSkinVertex& SoftVert = Section.SoftVertices[SoftVertIndex];
		for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
		{
			const uint16 InfluenceWeight = SoftVert.InfluenceWeights[InfluenceIndex];
			if (InfluenceWeight >= 1)
			{
				const FBoneIndexType BoneMapIndex = SoftVert.InfluenceBones[InfluenceIndex];
				const int32 ActualBoneIndex = Section.BoneMap[BoneMapIndex];
				Bones.Add(ActualBoneIndex);
			}
		}
	}
}