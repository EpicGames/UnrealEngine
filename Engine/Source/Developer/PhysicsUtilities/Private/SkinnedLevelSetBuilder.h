// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/WeightedLatticeImplicitObject.h"

class FSkinnedBoneTriangleCache;
class FSkeletalMeshModel;
class FSkeletalMeshRenderData;
struct FPhysAssetCreateParams;
class USkeletalMesh;
namespace Chaos
{
class FLevelSet;
}

class FSkinnedLevelSetBuilder : public Chaos::FWeightedLatticeImplicitObjectBuilder
{
public:

	FSkinnedLevelSetBuilder(const USkeletalMesh& InSkeletalMesh, const FSkinnedBoneTriangleCache& InTriangleCache, const int32 InRootBoneIndex);

	bool InitializeSkinnedLevelset(const FPhysAssetCreateParams& Params, const TArray<int32>& BoneIndices, TArray<uint32>& OutOrigIndices);
	void AddBoneInfluence(int32 PrimaryBoneIndex, const TArray<int32>& AllBonesForInfluence);

	FKSkinnedLevelSetElem CreateSkinnedLevelSetElem();

	void GetInfluencingBones(const TArray<uint32>& SkinnedVertexIndices, TSet<int32>& Bones);

private:
	// Input
	const USkeletalMesh& SkeletalMesh;
	const FSkinnedBoneTriangleCache& TriangleCache;
	const int32 RootBoneIndex;

	// Computed from Input
	FSkeletalMeshModel* StaticLODModel;
	const FSkeletalMeshRenderData* RenderData;

	// Output
	Chaos::FLevelSetPtr LevelSet;

	float GetWeightForIndices(const TSet<int32>& BoneIndices, uint32 VertIndex) const;
};
