// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ConvexDecompTool.h: Utility for turning graphics mesh into convex hulls.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

/** This pre-processor define specifies the default
	voxel resolution when performing V-HACD.
	The default value is one hundred thousands voxels.
    The valid range is between 10,000 to 16 million.
    Generally speaking 100,000 is more than enough
    precision for any mesh your might care to process.
*/
#ifndef DEFAULT_HACD_VOXEL_RESOLUTION
#define DEFAULT_HACD_VOXEL_RESOLUTION 100000
#endif

class UBodySetup;

/** 
 *	Utility for turning arbitrary mesh into convex hulls.
 *	@output		InBodySetup			BodySetup that will have its existing hulls removed and replaced with results of decomposition.
 *	@param		InVertices			Array of vertex positions of input mesh
 *	@param		InIndices			Array of triangle indices for input mesh
 *	@param		InAccuracy			Value between 0 and 1, controls how accurate hull generation is
 *	@param		InMaxHullVerts		Number of verts allowed in a hull
 */
PHYSICSUTILITIES_API void DecomposeMeshToHulls(UBodySetup* InBodySetup, const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, uint32 InHullCount, int32 InMaxHullVerts,uint32 InResolution= DEFAULT_HACD_VOXEL_RESOLUTION);

class IDecomposeMeshToHullsAsync
{
public:
	// Returns the current status message in the decomposition process.  An empty string is returns if the convex decomposition is complete.
	virtual const FString &GetCurrentStatus(void) const = 0;

	/**
	*	Utility for turning arbitrary mesh into convex hulls.  This can be called multiple times for multiple UBodySetup results and they will be done processed in sequential order
	*	@output		InBodySetup			BodySetup that will have its existing hulls removed and replaced with results of decomposition.
	*	@param		InVertices			Array of vertex positions of input mesh, whose ownership is transferred to the async task
	*	@param		InIndices			Array of triangle indices for input mesh, whose ownership is transferred to the async task
	*	@param		InAccuracy			Value between 0 and 1, controls how accurate hull generation is
	*	@param		InMaxHullVerts		Number of verts allowed in a hull
	*/
	virtual bool DecomposeMeshToHullsAsyncBegin(UBodySetup* InBodySetup, TArray<FVector3f>&& InVertices, TArray<uint32>&& InIndices, uint32 InHullCount, int32 InMaxHullVerts, uint32 InResolution = DEFAULT_HACD_VOXEL_RESOLUTION) = 0;

	// Returns true if the convex decomposition process has completed and results are ready.
	virtual bool IsComplete(void) = 0;

	// Release the async convex decomposition interface
	virtual void Release(void) = 0;
protected:
	virtual ~IDecomposeMeshToHullsAsync(void) = default;
};

// Creates the interface to the asynchronous convex decomposition tool chain
PHYSICSUTILITIES_API IDecomposeMeshToHullsAsync *CreateIDecomposeMeshToHullAsync(void);
