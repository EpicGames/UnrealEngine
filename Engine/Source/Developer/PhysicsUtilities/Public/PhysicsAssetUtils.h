// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Chaos/ChaosEngineInterface.h"
#include "PhysicsAssetUtils.generated.h"

class UBodySetup;
class UPhysicsAsset;
class UPhysicsConstraintTemplate;
class USkeletalMesh;
class USkeletalMeshComponent;

UENUM()
enum EPhysAssetFitGeomType : int
{
	EFG_Box					UMETA(DisplayName="Box"),
	EFG_Sphyl				UMETA(DisplayName="Capsule"),
	EFG_Sphere				UMETA(DisplayName="Sphere"),
	EFG_TaperedCapsule		UMETA(DisplayName="Tapered Capsule (Cloth Only)"),
	EFG_SingleConvexHull	UMETA(DisplayName="Single Convex Hull"),
	EFG_MultiConvexHull		UMETA(DisplayName="Multi Convex Hull"),
	EFG_LevelSet			UMETA(DisplayName="Level Set (Experimental, Cloth Only)"),
	EFG_SkinnedLevelSet		UMETA(DisplayName="Skinned Level Set (Experimental, Cloth Only)"),
	EFG_MLLevelSet			UMETA(Hidden),
	EFG_SkinnedTriangleMesh UMETA(Hidden),
};

UENUM()
enum EPhysAssetFitVertWeight : int
{
	EVW_AnyWeight			UMETA(DisplayName="Any Weight"),
	EVW_DominantWeight		UMETA(DisplayName="Dominant Weight"),
};

/** Parameters for PhysicsAsset creation */
USTRUCT()
struct FPhysAssetCreateParams
{
	GENERATED_BODY()

	FPhysAssetCreateParams()
	{
		MinBoneSize = 20.0f;
		MinWeldSize = KINDA_SMALL_NUMBER;
		GeomType = EFG_Sphyl;
		VertWeight = EVW_DominantWeight;
		bAlwaysUseVertices = true;
		bIncludeChildBones = true;
		bAutoOrientToBone = true;
		bCreateConstraints = true;
		bWalkPastSmall = true;
		bBodyForAll = false;
		bDisableCollisionsByDefault = true;
		AngularConstraintMode = ACM_Limited;
		HullCount = 4;
		MaxHullVerts = 16;
		LevelSetResolution = 8;
		LatticeResolution = 8;
		LodIndex = 0;
	}

	/** Bones that are shorter than this value will be ignored for body creation */
	UPROPERTY(EditAnywhere, Category = "Body Creation")
	float								MinBoneSize;

	/** Bones that are smaller than this value will be merged together for body creation */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Body Creation")
	float								MinWeldSize;

	/** The geometry type that should be used when creating bodies */
	UPROPERTY(EditAnywhere, Category = "Body Creation", meta=(DisplayName="Primitive Type"))
	TEnumAsByte<EPhysAssetFitGeomType> GeomType;

	/** How vertices are mapped to bones when approximating them with bodies */
	UPROPERTY(EditAnywhere, Category = "Body Creation", meta=(DisplayName="Vertex Weighting Type"))
	TEnumAsByte<EPhysAssetFitVertWeight> VertWeight;

	/** 
	 * If true, then mesh vertices are always used to calculate the collision shape, which can be slow. If false then 
	 * the simpler shapes will not use vertices, which can result in faster processing.
	 */
	UPROPERTY(EditAnywhere, Category = "Body Creation")
	bool								bAlwaysUseVertices;

	/**
	 * If true, then when mesh vertices are used, non-physical bones of the current bone will be included when
	 * calculating the collision shapes.
	 */
	UPROPERTY(EditAnywhere, Category = "Body Creation")
	bool								bIncludeChildBones;

	/** Whether to automatically orient the created bodies to their corresponding bones */
	UPROPERTY(EditAnywhere, Category = "Body Creation")
	bool								bAutoOrientToBone;

	/** Whether to create constraints between adjacent created bodies */
	UPROPERTY(EditAnywhere, Category = "Constraint Creation")
	bool								bCreateConstraints;

	/** Whether to skip small bones entirely (rather than merge them with adjacent bones) */
	UPROPERTY(EditAnywhere, Category = "Body Creation", meta=(DisplayName="Walk Past Small Bones"))
	bool								bWalkPastSmall;

	/** Forces creation of a body for each bone */
	UPROPERTY(EditAnywhere, Category = "Body Creation", meta=(DisplayName="Create Body for All Bones"))
	bool								bBodyForAll;

	/** Whether to disable collision of body with other bodies on creation */
	UPROPERTY(EditAnywhere, Category = "Body Creation")
	bool								bDisableCollisionsByDefault;

	/** The type of angular constraint to create between bodies */
	UPROPERTY(EditAnywhere, Category = "Constraint Creation", meta=(EditCondition="bCreateConstraints"))
	TEnumAsByte<EAngularConstraintMotion> AngularConstraintMode;

	/** The LOD index use to create the physics asset.*/
	UPROPERTY(EditAnywhere, Category = "Body Creation")
	int32								LodIndex;

	/** When creating multiple convex hulls, the maximum number that will be created. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Body Creation", meta = (EditCondition = "GeomType == EPhysAssetFitGeomType::EFG_MultiConvexHull", EditConditionHides))
	int32								HullCount;

	/** When creating convex hulls, the maximum verts that should be created */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Body Creation", meta = (EditCondition = "GeomType == EPhysAssetFitGeomType::EFG_MultiConvexHull || GeomType == EPhysAssetFitGeomType::EFG_SingleConvexHull", EditConditionHides))
	int32								MaxHullVerts;

	/** When creating level sets, the grid resolution to use */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Body Creation", 
		meta = (ClampMin = 1, UIMin = 10, UIMax = 100, ClampMax = 500, EditCondition = "GeomType == EPhysAssetFitGeomType::EFG_LevelSet || GeomType == EPhysAssetFitGeomType::EFG_SkinnedLevelSet", EditConditionHides))
	int32								LevelSetResolution;

	/** When creating skinned level sets, the embedding grid resolution to use*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Body Creation",
		meta = (ClampMin = 1, UIMin = 10, UIMax = 100, ClampMax = 500, EditCondition = "GeomType == EPhysAssetFitGeomType::EFG_SkinnedLevelSet", EditConditionHides))
	int32								LatticeResolution;
};

class UPhysicsAsset;
class UPhysicsConstraintTemplate;
struct FBoneVertInfo;

/** Collection of functions to create and setup PhysicsAssets */
namespace FPhysicsAssetUtils
{
	/**
	 * Given a USkeletalMesh, construct a new PhysicsAsset automatically, using the vertices weighted to each bone to calculate approximate collision geometry.
	 * Ball-and-socket joints will be created for every joint by default.
	 *
	 * @param	PhysicsAsset		The PhysicsAsset instance to setup
	 * @param	SkelMesh			The Skeletal Mesh to create the physics asset from
	 * @param	Params				Additional creation parameters
	 * @param	OutErrorMessage		Additional error information
	 * @param	bSetToMesh			Whether or not to apply the physics asset to SkelMesh immediately
	 */
	PHYSICSUTILITIES_API bool CreateFromSkeletalMesh(UPhysicsAsset* PhysicsAsset, USkeletalMesh* SkelMesh, const FPhysAssetCreateParams& Params, FText& OutErrorMessage, bool bSetToMesh = true, bool bShowProgress = true);

	/** Replaces any collision already in the BodySetup with an auto-generated one using the parameters provided.
	 * 
	 * @warning Certain physics geometry types, such as multi-convex hull, must recreate internal caches every time this function is called.
	 * If you find you're calling this function repeatedly for different bone indices on the same mesh,
	 * CreateFromSkeletalMesh or CreateCollisionsFromBones will provide better performance.
	 *
	 * @param	BodySetup              BodySetup to create the collision for
	 * @param	SkelMesh               The SkeletalMesh we create collision for
	 * @param	BoneIndex              Index of the bone the collision is created for
	 * @param	Params                 Additional parameters to control the creation 
	 * @param   Info                   The vertices to create the collision for
	 * @param   PhysicsAsset           If set, then this can/will be used to recursively find relevant child bones for collision generation
	 * @param   BoneIndicesInCreation  If set, then this can/will be used to identify bones that will be physical (for collision generation) 
	 *                                 and should not be included in another bone's collision
	 * @return  Returns true if successfully created collision from bone
	 */
	PHYSICSUTILITIES_API bool CreateCollisionFromBone(
		UBodySetup* BodySetup, USkeletalMesh* SkelMesh, int32 BoneIndex, const FPhysAssetCreateParams& Params,
		const FBoneVertInfo& Info, const UPhysicsAsset* PhysicsAsset,
		TConstArrayView<int32> BoneIndicesInCreation = TConstArrayView<int32>());

	/** Replaces any collision already in the BodySetup with an auto-generated one using the parameters provided.
	 * 
	 * @param	BodySetup					BodySetup to create the collision for
	 * @param	SkelMesh			The SkeletalMesh we create collision for
	 * @param	BoneIndices			Indices of the bones the collisions are created for
	 * @param	Params				Additional parameters to control the creation 
	 * @param	Info				The vertices to create the collision for
	 * @param   PhysicsAsset        If set, then this can/will be used to recursively find relevant child bones for collision generation
	 * @return  Returns true if successfully created collision from all specified bones
	 */
	PHYSICSUTILITIES_API bool CreateCollisionFromBones( UBodySetup* BodySetup, USkeletalMesh* SkelMesh, const TArray<int32>& BoneIndices, FPhysAssetCreateParams& Params, const FBoneVertInfo& Info, const UPhysicsAsset* PhysicsAsset );

	/** Replaces any collision already in the  with an auto-generated ones using the parameters provided.
	 *
	 * @param	PhysicsAsset		The PhysicsAsset instance to update
	 * @param	SkelMesh			The Skeletal Mesh to create the physics asset from
	 * @param	BodyIndices			Indices of the existing BodySetups the collisions are created for
	 * @param	Params				Additional parameters to control the creation
	 * @param	Infos				The vertices to create the collisions for (parallel array to BodyIndices)
	 * @param   OutSuccessfulBodyIndices Newly created BodySetup indices.
	 * @return  Returns true if successfully created collision from all specified bodies
	 */
	PHYSICSUTILITIES_API bool CreateCollisionsFromBones(UPhysicsAsset* PhysicsAsset, USkeletalMesh* SkelMesh, const TArray<int32>& BodyIndices, const FPhysAssetCreateParams& Params, const TArray<FBoneVertInfo>& Infos, TArray<int32>& OutSuccessfulBodyIndices);

	/**
	 * Given a USkeletalMesh, generate a Skinned Triangle Mesh primitive and add it to the root body (creating a new one if needed).
	 *
	 * @param	PhysicsAsset		The PhysicsAsset instance to update
	 * @param	SkelMesh			The Skeletal Mesh to create the physics asset from
	 * @param	LODIndex			SkelMesh LOD index to use.
	 * @param	Params				Additional creation parameters
	 * @param	OutErrorMessage		Additional error information
	 * @return  Index of body under which the new Skinned Triangle Mesh primitive was created (or INDEX_NONE if failure)
	 */
	PHYSICSUTILITIES_API int32 GenerateSkinnedTriangleMesh(UPhysicsAsset* PhysicsAsset, USkeletalMesh* SkelMesh, int32 LODIndex, const FPhysAssetCreateParams& Params, FText& OutErrorMessage, bool bShowProgress = true);
	/**
	 * Does a few things:
	 * - add any collision primitives from body2 into body1 (adjusting the tm of each).
	 * - reconnect any constraints between 'add body' to 'base body', destroying any between them.
	 * - update collision disable table for any pairs including 'add body'
	 */
	PHYSICSUTILITIES_API void WeldBodies(UPhysicsAsset* PhysAsset, int32 BaseBodyIndex, int32 AddBodyIndex, USkeletalMeshComponent* SkelComp);

	/**
	 * Creates a new constraint
	 *
	 * @param	PhysAsset			The PhysicsAsset to create the constraint for
	 * @param	InConstraintName	Name of the constraint
	 * @param	InConstraintSetup	Optional constraint setup
	 * @return  Returns the index of the newly created constraint.
	 **/
	PHYSICSUTILITIES_API int32 CreateNewConstraint(UPhysicsAsset* PhysAsset, FName InConstraintName, UPhysicsConstraintTemplate* InConstraintSetup = NULL);

	/**
	 * Destroys the specified constraint
	 *
	 * @param	PhysAsset			The PhysicsAsset for which the constraint should be destroyed
	 * @param	ConstraintIndex		The index of the constraint to destroy
	 */
	PHYSICSUTILITIES_API void DestroyConstraint(UPhysicsAsset* PhysAsset, int32 ConstraintIndex);

	/**
	 * Create a new BodySetup and default BodyInstance if there is not one for this body already.
	 *
	 * @param	PhysAsset			The PhysicsAsset to create the body for
	 * @param	InBodyName			Name of the new body
	 * @return	The Index of the newly created body.
	 */
	PHYSICSUTILITIES_API int32 CreateNewBody(UPhysicsAsset* const PhysAsset, const FName InBodyName, const FPhysAssetCreateParams& Params);
	
	/** 
	 * Destroys the specified body
	 *
	 * @param	PhysAsset			The PhysicsAsset for which the body should be destroyed
	 * @param	BodyIndex			Index of the body to destroy
	 */
	PHYSICSUTILITIES_API void DestroyBody(UPhysicsAsset* const PhysAsset, const int32 BodyIndex);

	/**
	 * Initialize a new BodySetup at a specified index in the physics asset and create a default BodyInstance if there is not one for this body already.
	 *
	 * @param	PhysAsset			The PhysicsAsset to create the body for
	 * @param	InBodyName			Name of the new body
	 * @param	Params				Parameters used when configuring the body
	 * @param	BodyIndex			Index of the body to initialize in the Physics Asset
	 */
	PHYSICSUTILITIES_API void RecreateBody(UPhysicsAsset* const PhysAsset, const FName InBodyName, const FPhysAssetCreateParams& Params, const int32 BodyIndex);

	/**
	* Whether or not Constraints are allowed to be created (due to asset list filtering)
	*/
	PHYSICSUTILITIES_API bool CanCreateConstraints();

	UE_DEPRECATED(5.8, "SanitizeRestrictedContent is no longer used.") PHYSICSUTILITIES_API void SanitizeRestrictedContent(UPhysicsAsset* PhysAsset);

	/**
	 * Compute a covariance matrix describing the distribution of points in the provided set
	 * 
	 * @param PointView An array view of a vector type to calculate the covariance matrix for
	 * @return Calculated covariance matrix
	 */
	template<typename RealType>
	FMatrix ComputeViewCovarianceMatrix(TArrayView<const UE::Math::TVector<RealType>, int> PointView);
	PHYSICSUTILITIES_API FMatrix ComputeCovarianceMatrix(const TArray<FVector3f>& Verts);

	extern template FMatrix ComputeViewCovarianceMatrix(TArrayView<const UE::Math::TVector<float>, int> PointView);
	extern template FMatrix ComputeViewCovarianceMatrix(TArrayView<const UE::Math::TVector<double>, int> PointView);

	/**
	 * Compute the dominant eigenvector of a provided matrix
	 * 
	 * @param InMatrix Matrix input
	 * @return Domininant eigenvector of InMatrix
	 */
	PHYSICSUTILITIES_API FVector ComputeEigenVector(const FMatrix& InMatrix);
};